// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class ClassViewer : ModuleRules
{
	public ClassViewer(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
                "AssetTools",
				"EditorWidgets",
				"GameProjectGeneration",
				"WorkspaceMenuStructure"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
				"Core",
				"CoreUObject",
				"Engine",
                "InputCore",
				"Slate",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
				"PropertyEditor",
				"ContentBrowserData",
                "Settings",
            }
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
                "AssetTools",
				"EditorWidgets",
				"GameProjectGeneration",
			}
		);
	}
}
