// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClassViewerFilter.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "ClassViewerModule.h"
#include "ClassViewerNode.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Brush.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "Settings/ClassViewerSettings.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UnloadedBlueprintData.h"

EFilterReturn::Type FClassViewerFilterFuncs::IfInChildOfClassesSet(TSet< const UClass* >& InSet, const UClass* InClass)
{
	check(InClass);

	if (InSet.Num())
	{
		// If a class is a child of any classes on this list, it will be allowed onto the list, unless it also appears on a disallowed list.
		for (auto CurClassIt = InSet.CreateConstIterator(); CurClassIt; ++CurClassIt)
		{
			if (InClass->IsChildOf(*CurClassIt))
			{
				return EFilterReturn::Passed;
			}
		}

		return EFilterReturn::Failed;
	}

	// Since there are none on this list, return that there is no items.
	return EFilterReturn::NoItems;
}

EFilterReturn::Type FClassViewerFilterFuncs::IfInChildOfClassesSet(TSet< const UClass* >& InSet, const TSharedPtr< const IUnloadedBlueprintData > InClass)
{
	check(InClass.IsValid());

	if (InSet.Num())
	{
		// If a class is a child of any classes on this list, it will be allowed onto the list, unless it also appears on a disallowed list.
		for (auto CurClassIt = InSet.CreateConstIterator(); CurClassIt; ++CurClassIt)
		{
			if (InClass->IsChildOf(*CurClassIt))
			{
				return EFilterReturn::Passed;
			}
		}

		return EFilterReturn::Failed;
	}

	// Since there are none on this list, return that there is no items.
	return EFilterReturn::NoItems;
}

EFilterReturn::Type FClassViewerFilterFuncs::IfMatchesAllInChildOfClassesSet(TSet< const UClass* >& InSet, const UClass* InClass)
{
	check(InClass);

	if (InSet.Num())
	{
		// If a class is a child of any classes on this list, it will be allowed onto the list, unless it also appears on a disallowed list.
		for (auto CurClassIt = InSet.CreateConstIterator(); CurClassIt; ++CurClassIt)
		{
			if (!InClass->IsChildOf(*CurClassIt))
			{
				// Since it doesn't match one, it fails.
				return EFilterReturn::Failed;
			}
		}

		// It matches all of them, so it passes.
		return EFilterReturn::Passed;
	}

	// Since there are none on this list, return that there is no items.
	return EFilterReturn::NoItems;
}

EFilterReturn::Type FClassViewerFilterFuncs::IfMatchesAllInChildOfClassesSet(TSet< const UClass* >& InSet, const TSharedPtr< const IUnloadedBlueprintData > InClass)
{
	check(InClass.IsValid());

	if (InSet.Num())
	{
		// If a class is a child of any classes on this list, it will be allowed onto the list, unless it also appears on a disallowed list.
		for (auto CurClassIt = InSet.CreateConstIterator(); CurClassIt; ++CurClassIt)
		{
			if (!InClass->IsChildOf(*CurClassIt))
			{
				// Since it doesn't match one, it fails.
				return EFilterReturn::Failed;
			}
		}

		// It matches all of them, so it passes.
		return EFilterReturn::Passed;
	}

	// Since there are none on this list, return that there is no items.
	return EFilterReturn::NoItems;
}

EFilterReturn::Type FClassViewerFilterFuncs::IfMatchesAll_ObjectsSetIsAClass(TSet< const UObject* >& InSet, const UClass* InClass)
{
	check(InClass);

	if (InSet.Num())
	{
		// If a class is a child of any classes on this list, it will be allowed onto the list, unless it also appears on a disallowed list.
		for (auto CurClassIt = InSet.CreateConstIterator(); CurClassIt; ++CurClassIt)
		{
			if (!(*CurClassIt)->IsA(InClass))
			{
				// Since it doesn't match one, it fails.
				return EFilterReturn::Failed;
			}
		}

		// It matches all of them, so it passes.
		return EFilterReturn::Passed;
	}

	// Since there are none on this list, return that there is no items.
	return EFilterReturn::NoItems;
}

EFilterReturn::Type FClassViewerFilterFuncs::IfMatchesAll_ObjectsSetIsAClass(TSet< const UObject* >& InSet, const TSharedPtr< const IUnloadedBlueprintData > InClass)
{
	check(InClass.IsValid());

	if (InSet.Num())
	{
		// If a class is a child of any classes on this list, it will be allowed onto the list, unless it also appears on a disallowed list.
		for (auto CurClassIt = InSet.CreateConstIterator(); CurClassIt; ++CurClassIt)
		{
			if (!(*CurClassIt)->IsA(UBlueprintGeneratedClass::StaticClass()))
			{
				// Since it doesn't match one, it fails.
				return EFilterReturn::Failed;
			}
		}

		// It matches all of them, so it passes.
		return EFilterReturn::Passed;
	}

	// Since there are none on this list, return that there is no items.
	return EFilterReturn::NoItems;
}

EFilterReturn::Type FClassViewerFilterFuncs::IfMatchesAll_ClassesSetIsAClass(TSet< const UClass* >& InSet, const UClass* InClass)
{
	check(InClass);

	if (InSet.Num())
	{
		// If a class is a child of any classes on this list, it will be allowed onto the list, unless it also appears on a disallowed list.
		for (auto CurClassIt = InSet.CreateConstIterator(); CurClassIt; ++CurClassIt)
		{
			const UObject* Object = *CurClassIt;
			if (!Object->IsA(InClass))
			{
				// Since it doesn't match one, it fails.
				return EFilterReturn::Failed;
			}
		}

		// It matches all of them, so it passes.
		return EFilterReturn::Passed;
	}

	// Since there are none on this list, return that there is no items.
	return EFilterReturn::NoItems;
}

EFilterReturn::Type FClassViewerFilterFuncs::IfMatchesAll_ClassesSetIsAClass(TSet< const UClass* >& InSet, const TSharedPtr< const IUnloadedBlueprintData > InClass)
{
	check(InClass.IsValid());

	if (InSet.Num())
	{
		// If a class is a child of any classes on this list, it will be allowed onto the list, unless it also appears on a disallowed list.
		for (auto CurClassIt = InSet.CreateConstIterator(); CurClassIt; ++CurClassIt)
		{
			const UObject* Object = *CurClassIt;
			if (!Object->IsA(UBlueprintGeneratedClass::StaticClass()))
			{
				// Since it doesn't match one, it fails.
				return EFilterReturn::Failed;
			}
		}

		// It matches all of them, so it passes.
		return EFilterReturn::Passed;
	}

	// Since there are none on this list, return that there is no items.
	return EFilterReturn::NoItems;
}

EFilterReturn::Type FClassViewerFilterFuncs::IfMatches_ClassesSetIsAClass(TSet< const UClass* >& InSet, const UClass* InClass)
{
	check(InClass);

	if (InSet.Num())
	{
		// If a class is a child of any classes on this list, it will be allowed onto the list, unless it also appears on a disallowed list.
		for (auto CurClassIt = InSet.CreateConstIterator(); CurClassIt; ++CurClassIt)
		{
			const UObject* Object = *CurClassIt;
			if (Object->IsA(InClass))
			{
				return EFilterReturn::Passed;
			}
		}

		return EFilterReturn::Failed;
	}

	// Since there are none on this list, return that there is no items.
	return EFilterReturn::NoItems;
}

EFilterReturn::Type FClassViewerFilterFuncs::IfMatches_ClassesSetIsAClass(TSet< const UClass* >& InSet, const TSharedPtr< const IUnloadedBlueprintData > InClass)
{
	check(InClass.IsValid());

	if (InSet.Num())
	{
		// If a class is a child of any classes on this list, it will be allowed onto the list, unless it also appears on a disallowed list.
		for (auto CurClassIt = InSet.CreateConstIterator(); CurClassIt; ++CurClassIt)
		{
			const UObject* Object = *CurClassIt;
			if (Object->IsA(UBlueprintGeneratedClass::StaticClass()))
			{
				return EFilterReturn::Passed;
			}
		}

		return EFilterReturn::Failed;
	}

	// Since there are none on this list, return that there is no items.
	return EFilterReturn::NoItems;
}

EFilterReturn::Type FClassViewerFilterFuncs::IfInClassesSet(TSet< const UClass* >& InSet, const UClass* InClass)
{
	check(InClass);

	if (InSet.Num())
	{
		for (auto CurClassIt = InSet.CreateConstIterator(); CurClassIt; ++CurClassIt)
		{
			if (InClass == *CurClassIt)
			{
				return EFilterReturn::Passed;
			}
		}
		return EFilterReturn::Failed;
	}

	// Since there are none on this list, return that there is no items.
	return EFilterReturn::NoItems;
}

EFilterReturn::Type FClassViewerFilterFuncs::IfInClassesSet(TSet< const UClass* >& InSet, const TSharedPtr< const IUnloadedBlueprintData > InClass)
{
	check(InClass.IsValid());

	if (InSet.Num())
	{
		for (auto CurClassIt = InSet.CreateConstIterator(); CurClassIt; ++CurClassIt)
		{
			const TSharedPtr<const FUnloadedBlueprintData> UnloadedBlueprintData = StaticCastSharedPtr<const FUnloadedBlueprintData>(InClass);
			if (*UnloadedBlueprintData->GetClassViewerNode().Pin()->GetClassName() == (*CurClassIt)->GetName())
			{
				return EFilterReturn::Passed;
			}
		}
		return EFilterReturn::Failed;
	}

	// Since there are none on this list, return that there is no items.
	return EFilterReturn::NoItems;
}

/** Checks if a particular class is a brush.
*	@param InClass				The Class to check.
*	@return Returns true if the class is a brush.
*/
static bool IsBrush(const UClass* InClass)
{
	return InClass->IsChildOf(ABrush::StaticClass());
}

static bool IsBrush(const TSharedRef<const IUnloadedBlueprintData>& InBlueprintData)
{
	return InBlueprintData->IsChildOf(ABrush::StaticClass());
}

/** Checks if a particular class is placeable.
*	@param InClass				The Class to check.
*	@return Returns true if the class is placeable.
*/
static bool IsPlaceable(const UClass* InClass)
{
	return !InClass->HasAnyClassFlags(CLASS_Abstract | CLASS_NotPlaceable) 
		&& InClass->IsChildOf(AActor::StaticClass());
}

static bool IsPlaceable(const TSharedRef<const IUnloadedBlueprintData>& InBlueprintData)
{
	return !InBlueprintData->HasAnyClassFlags(CLASS_Abstract | CLASS_NotPlaceable)
		&& InBlueprintData->IsChildOf(AActor::StaticClass());
}

/** Util class to checks if a particular class can be made into a Blueprint, ignores deprecation
 *
 * @param InClass	The class to verify can be made into a Blueprint
 * @return			true if the class can be made into a Blueprint
 */
static bool CanCreateBlueprintOfClass(UClass* InClass)
{
	// Temporarily remove the deprecated flag so we can check if it is valid for
	bool bIsClassDeprecated = InClass->HasAnyClassFlags(CLASS_Deprecated);
	InClass->ClassFlags &= ~CLASS_Deprecated;

	bool bCanCreateBlueprintOfClass = FKismetEditorUtilities::CanCreateBlueprintOfClass(InClass);

	// Reassign the deprecated flag if it was previously assigned
	if (bIsClassDeprecated)
	{
		InClass->ClassFlags |= CLASS_Deprecated;
	}

	return bCanCreateBlueprintOfClass;
}

/** Checks if a node is a blueprint base or not.
*	@param	InNode	The node to check if it is a blueprint base.
*	@return			true if the class is a blueprint base.
*/
static bool CheckIfBlueprintBase(const TSharedRef<const IUnloadedBlueprintData>& InBlueprintData)
{
	if (InBlueprintData->IsNormalBlueprintType())
	{
		bool bAllowDerivedBlueprints = false;
		GConfig->GetBool(TEXT("Kismet"), TEXT("AllowDerivedBlueprints"), /*out*/ bAllowDerivedBlueprints, GEngineIni);

		return bAllowDerivedBlueprints;
	}
	return false;
}

/** Checks if the TestString passes the filter.
*	@param InTestString			The string to test against the filter.
*	@param InTextFilter			Compiled text filter to apply.
*
*	@return	true if it passes the filter.
*/
static bool PassesTextFilter(const FString& InTestString, const TSharedRef<FTextFilterExpressionEvaluator>& InTextFilter)
{
	return InTextFilter->TestTextFilter(FBasicStringFilterExpressionContext(InTestString));
}

/** Returns true if the given class is a REINST class (starts with the 'REINST_' prefix) */
static bool IsReinstClass(const UClass* Class)
{
	static const FString ReinstPrefix = TEXT("REINST");
	return Class && Class->GetFName().ToString().StartsWith(ReinstPrefix);
}

FClassViewerFilter::FClassViewerFilter(const FClassViewerInitializationOptions& InInitOptions) :
	TextFilter(MakeShared<FTextFilterExpressionEvaluator>(ETextFilterExpressionEvaluatorMode::BasicString)),
	FilterFunctions(MakeShared<FClassViewerFilterFuncs>()),
	AssetRegistry(FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get())
{
	// Create a game-specific filter, if the referencing property/assets were supplied
	if (GEditor)
	{
		FAssetReferenceFilterContext AssetReferenceFilterContext;
		AssetReferenceFilterContext.AddReferencingAssets(InInitOptions.AdditionalReferencingAssets);
		AssetReferenceFilterContext.AddReferencingAssetsFromPropertyHandle(InInitOptions.PropertyHandle);
		AssetReferenceFilter = GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
	}
	
	// MetaData class filter
	FProperty* Property = InInitOptions.PropertyHandle.IsValid() ? InInitOptions.PropertyHandle->GetProperty() : nullptr;
	if (Property && Property->GetOwnerProperty()->HasMetaData("GetClassFilter"))
	{
		const FString& GetClassFilterFunctionName = Property->GetOwnerProperty()->GetMetaData("GetClassFilter");
		if (!GetClassFilterFunctionName.IsEmpty())
		{
			if (const UClass* OuterBaseClass = InInitOptions.PropertyHandle->GetOuterBaseClass())
			{
				UObject* OuterBaseCDO = OuterBaseClass->GetDefaultObject<UObject>();
				const UFunction* GetClassFilterFunction = OuterBaseCDO ? OuterBaseCDO->FindFunction(*GetClassFilterFunctionName) : nullptr;
				if (ensure(GetClassFilterFunction && GetClassFilterFunction->HasAllFunctionFlags(FUNC_Static)))
				{
					FilterDelegate = FOnShouldFilterClass::CreateUFunction(OuterBaseCDO, GetClassFilterFunction->GetFName());
				}
			}
		}
	}
}

bool FClassViewerFilter::IsNodeAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<FClassViewerNode>& InNode, const bool bCheckTextFilter)
{
	if (InNode->Class.IsValid())
	{
		return IsClassAllowed(InInitOptions, InNode->Class.Get(), FilterFunctions, bCheckTextFilter);
	}
	else if (InInitOptions.bShowUnloadedBlueprints && InNode->UnloadedBlueprintData.IsValid())
	{
		return IsUnloadedClassAllowed(InInitOptions, InNode->UnloadedBlueprintData.ToSharedRef(), FilterFunctions, bCheckTextFilter);
	}

	return false;
}

bool FClassViewerFilter::IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<class FClassViewerFilterFuncs> InFilterFuncs)
{
	const bool bCheckTextFilter = true;
	return IsClassAllowed(InInitOptions, InClass, InFilterFuncs, bCheckTextFilter);
}

bool FClassViewerFilter::IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<class FClassViewerFilterFuncs> InFilterFuncs, const bool bCheckTextFilter)
{
	if (InInitOptions.bIsActorsOnly && !InClass->IsChildOf(AActor::StaticClass()))
	{
		return false;
	}

	if (InInitOptions.bIsBlueprintBaseOnly && !CanCreateBlueprintOfClass(const_cast<UClass*>(InClass)))
	{
		return false;
	}

	if (InInitOptions.bEditorClassesOnly && !IsEditorOnlyObject(InClass))
	{
		return false;
	}

	// Determine if we allow any developer folder classes, if so determine if this class is in one of the allowed developer folders.
	static const FString DeveloperPathWithSlash = FPackageName::FilenameToLongPackageName(FPaths::GameDevelopersDir());
	static const FString UserDeveloperPathWithSlash = FPackageName::FilenameToLongPackageName(FPaths::GameUserDeveloperDir());
	const FString GeneratedClassPathString = InClass->GetPathName();

	const UClassViewerSettings* ClassViewerSettings = GetDefault<UClassViewerSettings>();

	EClassViewerDeveloperType AllowedDeveloperType = ClassViewerSettings->DeveloperFolderType;
	if (AllowedDeveloperType == EClassViewerDeveloperType::CVDT_None)
	{
		if (GeneratedClassPathString.StartsWith(DeveloperPathWithSlash))
		{
			return false;
		}
	}
	else if (AllowedDeveloperType == EClassViewerDeveloperType::CVDT_CurrentUser)
	{
		if (GeneratedClassPathString.StartsWith(DeveloperPathWithSlash) && !GeneratedClassPathString.StartsWith(UserDeveloperPathWithSlash))
		{
			return false;
		}
	}
	
	// The INI files declare classes and folders that are considered internal only. Does this class match any of those patterns?
	// INI path: /Script/ClassViewer.ClassViewerProjectSettings
	if (!ClassViewerSettings->DisplayInternalClasses)
	{
		for (const FDirectoryPath& DirPath : InternalPaths)
		{
			if (GeneratedClassPathString.StartsWith(DirPath.Path))
			{
				return false;
			}
		}

		for (const UClass* Class : InternalClasses)
		{
			if (InClass->IsChildOf(Class))
			{
				return false;
			}
		}
	}

	// The INI files can contain a list of globally allowed classes - if it does, then only classes whose names match will be shown.
	if (ClassViewerSettings->AllowedClasses.Num() > 0)
	{
		if (!ClassViewerSettings->AllowedClasses.Contains(GeneratedClassPathString))
		{
			return false;
		}
	}

	if (InInitOptions.bIsPlaceableOnly)
	{
		if (!IsPlaceable(InClass) || 
			!(InInitOptions.Mode == EClassViewerMode::ClassPicker || !IsBrush(InClass)))
		{
			return false;
		}
	}

	// REINST classes cannot be used in any class viewer. 
	if (IsReinstClass(InClass))
	{
		return false;
	}

	if (FilterDelegate.IsBound())
	{
		if (FilterDelegate.Execute(InClass))
		{
			return false;
		}
	}
	
	for (const TSharedRef<IClassViewerFilter>& CustomFilter : InInitOptions.ClassFilters)
	{
		if (!CustomFilter->IsClassAllowed(InInitOptions, InClass, FilterFunctions))
		{
			return false;
		}
	}

	if (const TSharedPtr<IClassViewerFilter>& GlobalClassFilter = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").GetGlobalClassViewerFilter())
	{
		if (!GlobalClassFilter->IsClassAllowed(InInitOptions, InClass, FilterFunctions))
		{
			return false;
		}
	}

	if (bCheckTextFilter && (TextFilter->GetFilterType() != ETextFilterExpressionType::Empty))
	{
		FString ClassNameWithCppPrefix = FString::Printf(TEXT("%s%s"), InClass->GetPrefixCPP(), *InClass->GetName());
		bool bPassesTextFilter = PassesTextFilter(InClass->GetName(), TextFilter) || PassesTextFilter(ClassNameWithCppPrefix, TextFilter) || PassesTextFilter(InClass->GetDisplayNameText().ToString(), TextFilter);

		// If the class is deprecated, try searching without the deprecated name inserted, in case a user typed a string
		if (!bPassesTextFilter && InClass->HasAnyClassFlags(CLASS_Deprecated))
		{
			ClassNameWithCppPrefix.RemoveAt(1, 11);
			bPassesTextFilter = PassesTextFilter(ClassNameWithCppPrefix, TextFilter);
		}

		if (!bPassesTextFilter)
		{
			return false;
		}
	}

	if (AssetReferenceFilter.IsValid() && !InClass->IsNative())
	{
		// This check is very slow as it scans all the asset tags
		if (!AssetReferenceFilter->PassesFilter(FAssetData(InClass)))
		{
			return false;
		}
	}

	return true;
}

bool FClassViewerFilter::IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs)
{
	const bool bCheckTextFilter = true;
	return IsUnloadedClassAllowed(InInitOptions, InUnloadedClassData, InFilterFuncs, bCheckTextFilter);
}

bool FClassViewerFilter::IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs, const bool bCheckTextFilter)
{
	if (InInitOptions.bIsActorsOnly && !InUnloadedClassData->IsChildOf(AActor::StaticClass()))
	{
		return false;
	}

	const bool bIsBlueprintBase = CheckIfBlueprintBase(InUnloadedClassData);
	if (InInitOptions.bIsBlueprintBaseOnly && !bIsBlueprintBase)
	{
		return false;
	}
	// TODO: There is currently no good way to handle bEditorClassesOnly for unloaded blueprints

	// Determine if we allow any developer folder classes, if so determine if this class is in one of the allowed developer folders.
	static const FString DeveloperPathWithSlash = FPackageName::FilenameToLongPackageName(FPaths::GameDevelopersDir());
	static const FString UserDeveloperPathWithSlash = FPackageName::FilenameToLongPackageName(FPaths::GameUserDeveloperDir());
	FTopLevelAssetPath UnloadedAssetPath = InUnloadedClassData->GetClassPathName();
	FString GeneratedClassPathString = UnloadedAssetPath.ToString();

	const UClassViewerSettings* ClassViewerSettings = GetDefault<UClassViewerSettings>();
	EClassViewerDeveloperType AllowedDeveloperType = ClassViewerSettings->DeveloperFolderType;
	if (AllowedDeveloperType == EClassViewerDeveloperType::CVDT_None)
	{
		if (GeneratedClassPathString.StartsWith(DeveloperPathWithSlash))
		{
			return false;
		}
	}
	else if (AllowedDeveloperType == EClassViewerDeveloperType::CVDT_CurrentUser)
	{
		if (GeneratedClassPathString.StartsWith(DeveloperPathWithSlash) && !GeneratedClassPathString.StartsWith(UserDeveloperPathWithSlash))
		{
			return false;
		}
	}

	// The INI files declare classes and folders that are considered internal only. Does this class match any of those patterns?
	// INI path: /Script/ClassViewer.ClassViewerProjectSettings
	if (!ClassViewerSettings->DisplayInternalClasses)
	{
		for (const FDirectoryPath& DirPath : InternalPaths)
		{
			if (GeneratedClassPathString.StartsWith(DirPath.Path))
			{
				return false;
			}
		}
	}

	// The INI files can contain a list of globally allowed classes - if it does, then only classes whose names match will be shown.
	if (ClassViewerSettings->AllowedClasses.Num() > 0)
	{
		if (!ClassViewerSettings->AllowedClasses.Contains(GeneratedClassPathString))
		{
			return false;
		}
	}

	if (InInitOptions.bIsPlaceableOnly)
	{
		if (!IsPlaceable(InUnloadedClassData) ||
			!(InInitOptions.Mode == EClassViewerMode::ClassPicker || !IsBrush(InUnloadedClassData)))
		{
			return false;
		}
	}

	for (const TSharedRef<IClassViewerFilter>& CustomFilter : InInitOptions.ClassFilters)
	{
		if (!CustomFilter->IsUnloadedClassAllowed(InInitOptions, InUnloadedClassData, FilterFunctions))
		{
			return false;
		}
	}

	if (const TSharedPtr<IClassViewerFilter>& GlobalClassFilter = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").GetGlobalClassViewerFilter())
	{
		if (!GlobalClassFilter->IsUnloadedClassAllowed(InInitOptions, InUnloadedClassData, FilterFunctions))
		{
			return false;
		}
	}

	if (bCheckTextFilter && !PassesTextFilter(*InUnloadedClassData->GetClassName().Get(), TextFilter))
	{
		return false;
	}

	if (AssetReferenceFilter.IsValid() && bIsBlueprintBase)
	{
		// This query is slow so should be done last
		const bool bOnlyIncludeOnDiskAssets = true; // If it was in memory it would have used the other filter function
		FAssetData BlueprintAssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(UnloadedAssetPath), bOnlyIncludeOnDiskAssets);
		if (!BlueprintAssetData.IsValid())
		{
			FString BlueprintAssetPath = GeneratedClassPathString.LeftChop(2); // Chop off _C
			BlueprintAssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(BlueprintAssetPath), bOnlyIncludeOnDiskAssets);
		}

		if (BlueprintAssetData.IsValid())
		{
			if (!AssetReferenceFilter->PassesFilter(BlueprintAssetData))
			{
				return false;
			}
		}
		else
		{
			UE_LOGF(LogEditorClassViewer, Warning, "Blueprint class cannot be found: %ls", *GeneratedClassPathString);
		}
	}

	return true;
}
