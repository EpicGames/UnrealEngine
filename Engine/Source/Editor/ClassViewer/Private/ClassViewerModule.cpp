// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClassViewerModule.h"

#include "ClassViewerFilter.h"
#include "ClassViewerProjectSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "ISettingsModule.h"
#include "Internationalization/Internationalization.h"
#include "Modules/ModuleManager.h"
#include "SClassViewer.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "ClassViewer"

IMPLEMENT_MODULE( FClassViewerModule, ClassViewer );

DEFINE_LOG_CATEGORY(LogEditorClassViewer);

namespace ClassViewerModule
{
	static const FName ClassViewerApp = FName("ClassViewerApp");
}

TSharedRef<SDockTab> CreateClassPickerTab( const FSpawnTabArgs& Args )
{
	FClassViewerInitializationOptions InitOptions;
	InitOptions.Mode = EClassViewerMode::ClassBrowsing;
	InitOptions.DisplayMode = EClassViewerDisplayMode::TreeView;

	return SNew(SDockTab)
		.TabRole( ETabRole::NomadTab )
		[
			SNew( SClassViewer, InitOptions )
			.OnClassPickedDelegate(FOnClassPicked())
		];
}


void FClassViewerModule::StartupModule()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner( ClassViewerModule::ClassViewerApp, FOnSpawnTab::CreateStatic( &CreateClassPickerTab ) )
		.SetDisplayName( NSLOCTEXT("ClassViewerApp", "TabTitle", "Class Viewer") )
		.SetTooltipText( NSLOCTEXT("ClassViewerApp", "TooltipText", "Displays all classes that exist within this project.") )
		.SetGroup( WorkspaceMenu::GetMenuStructure().GetToolsCategory() )
		.SetIcon( FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassViewer.TabIcon") );

	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule != nullptr)
	{
		// ClassViewer Editor Settings
		SettingsModule->RegisterSettings("Project", "Editor", "ClassViewer",
			LOCTEXT("ClassViewerSettingsName", "Class Viewer"),
			LOCTEXT("ClassViewerSettingsDescription", "Configure options for the Class Viewer."),
			GetMutableDefault<UClassViewerProjectSettings>()
			);
	}

}

void FClassViewerModule::ShutdownModule()
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner( ClassViewerModule::ClassViewerApp );
	}

	// Unregister the setting
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
		SettingsModule->UnregisterSettings("Project", "Editor", "ClassViewer");
	}

	SClassViewer::DestroyClassHierarchy();
}


/**
 * Creates a class viewer widget
 *
 * @param	InitOptions						Programmer-driven configuration for this widget instance
 * @param	OnClassPickedDelegate			Optional callback when a class is selected in 'class picking' mode
 *
 * @return	New class viewer widget
 */
TSharedRef<SWidget> FClassViewerModule::CreateClassViewer(const FClassViewerInitializationOptions& InitOptions, const FOnClassPicked& OnClassPickedDelegate )
{
	return SNew( SClassViewer, InitOptions )
			.OnClassPickedDelegate(OnClassPickedDelegate);
}

TSharedRef<IClassViewerFilter> FClassViewerModule::CreateClassFilter(const FClassViewerInitializationOptions& InitOptions)
{
	return TSharedRef<IClassViewerFilter>(new FClassViewerFilter(InitOptions));
}

TSharedRef<FClassViewerFilterFuncs> FClassViewerModule::CreateFilterFuncs()
{
	return TSharedRef<FClassViewerFilterFuncs>(new FClassViewerFilterFuncs());
}

void FClassViewerModule::RegisterGlobalClassViewerFilter(const TSharedRef<IClassViewerFilter>& Filter)
{
	GlobalClassViewerFilter = Filter;
	OnGlobalClassViewerFilterModified.Broadcast();
}

const TSharedPtr<IClassViewerFilter>& FClassViewerModule::GetGlobalClassViewerFilter()
{
	return GlobalClassViewerFilter;
}

#undef LOCTEXT_NAMESPACE
