// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClassViewerNode.h"

#include "ClassViewerFilter.h"
#include "Engine/Blueprint.h"
#include "Engine/Brush.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/StringFormatArg.h"
#include "PropertyHandle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"

FClassViewerNode::FClassViewerNode(UClass* InClass)
{
	Class = InClass;
	ClassName = MakeShareable(new FString(Class->GetName()));
	ClassDisplayName = MakeShareable(new FString(Class->GetDisplayNameText().ToString()));
	ClassPath = Class->GetPathName();

	if (Class->GetSuperClass())
	{
		ParentClassPath = Class->GetSuperClass()->GetClassPathName();
	}

	if (Class->ClassGeneratedBy && Class->ClassGeneratedBy->IsA(UBlueprint::StaticClass()))
	{
		Blueprint = Cast<UBlueprint>(Class->ClassGeneratedBy);
	}
	else
	{
		Blueprint = nullptr;
	}

	bPassesFilter = false;
	bPassesFilterRegardlessTextFilter = false;
}

FClassViewerNode::FClassViewerNode(const FString& InClassName, const FString& InClassDisplayName)
{
	ClassName = MakeShareable(new FString(InClassName));
	ClassDisplayName = MakeShareable(new FString(InClassDisplayName));
	bPassesFilter = false;
	bPassesFilterRegardlessTextFilter = false;

	Class = nullptr;
	Blueprint = nullptr;
}

FClassViewerNode::FClassViewerNode( const FClassViewerNode& InCopyObject)
{
	ClassName = InCopyObject.ClassName;
	ClassDisplayName = InCopyObject.ClassDisplayName;
	bPassesFilter = InCopyObject.bPassesFilter;
	bPassesFilterRegardlessTextFilter = InCopyObject.bPassesFilterRegardlessTextFilter;

	Class = InCopyObject.Class;
	Blueprint = InCopyObject.Blueprint;
	
	UnloadedBlueprintData = InCopyObject.UnloadedBlueprintData;

	ClassPath = InCopyObject.ClassPath;
	ParentClassPath = InCopyObject.ParentClassPath;
	ClassName = InCopyObject.ClassName;
	BlueprintAssetPath = InCopyObject.BlueprintAssetPath;

	// We do not want to copy the child list, do not add it. It should be the only item missing.
}

/**
 * Adds the specified child to the node.
 *
 * @param	Child							The child to be added to this node for the tree.
 */
void FClassViewerNode::AddChild( TSharedPtr<FClassViewerNode> Child )
{
	check(Child.IsValid());
	Child->ParentNode = AsShared();
	ChildrenList.Add(Child);
}

bool FClassViewerNode::IsRestricted() const
{
	return PropertyHandle.IsValid() && PropertyHandle->IsRestricted(*ClassName);
}

TSharedPtr<FString> FClassViewerNode::GetClassName(EClassViewerNameTypeToDisplay NameType) const
{
	switch (NameType)
	{
	case EClassViewerNameTypeToDisplay::ClassName:
		return ClassName;
	case EClassViewerNameTypeToDisplay::DisplayName:
		return ClassDisplayName;
	case EClassViewerNameTypeToDisplay::Dynamic:
		FString CombinedName;
		FString SanitizedName = FName::NameToDisplayString(*ClassName.Get(), false);
		if (ClassDisplayName.IsValid() && !ClassDisplayName->IsEmpty() && !ClassDisplayName->Equals(SanitizedName) && !ClassDisplayName->Equals(*ClassName.Get()))
		{
			TArray<FStringFormatArg> Args;
			Args.Add(*ClassName.Get());
			Args.Add(*ClassDisplayName.Get());
			CombinedName = FString::Format(TEXT("{0} ({1})"), Args);
		}
		else
		{
			CombinedName = *ClassName.Get();
		}
		return MakeShareable(new FString(CombinedName));
	}

	ensureMsgf(false, TEXT("FClassViewerNode::GetClassName called with invalid name type."));
	return ClassName;
}

bool FClassViewerNode::IsClassPlaceable() const
{
	const UClass* LoadedClass = Class.Get();
	if (LoadedClass)
	{
		const bool bPlaceableFlags = !LoadedClass->HasAnyClassFlags(CLASS_Abstract | CLASS_NotPlaceable);
		const bool bBasedOnActor = LoadedClass->IsChildOf(AActor::StaticClass());
		const bool bNotABrush = !LoadedClass->IsChildOf(ABrush::StaticClass());
		return bPlaceableFlags && bBasedOnActor && bNotABrush;
	}

	if (UnloadedBlueprintData.IsValid())
	{
		const bool bPlaceableFlags = !UnloadedBlueprintData->HasAnyClassFlags(CLASS_Abstract | CLASS_NotPlaceable);
		const bool bBasedOnActor = UnloadedBlueprintData->IsChildOf(AActor::StaticClass());
		const bool bNotABrush = !UnloadedBlueprintData->IsChildOf(ABrush::StaticClass());
		return bPlaceableFlags && bBasedOnActor && bNotABrush;
	}

	return false;
}

bool FClassViewerNode::IsBlueprintClass() const
{
	return !BlueprintAssetPath.IsNull();
}

bool FClassViewerNode::IsEditorOnlyClass() const
{
	return Class.IsValid() && IsEditorOnlyObject(Class.Get());
}

TSharedPtr< FClassViewerNode > FClassViewerNode::GetParentNode() const
{
	return ParentNode.Pin();
}
