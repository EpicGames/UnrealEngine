// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClassViewerModule.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/WeakObjectPtrTemplates.h"

class UBlueprint;
class UClass;

class FClassViewerNode : public TSharedFromThis<FClassViewerNode>
{
public:
	FClassViewerNode(UClass* Class);

	/**
	 * Creates a node for the widget's tree.
	 *
	 * @param	InClassName						The name of the class this node represents.
	 * @param	InClassDisplayName				The display name of the class this node represents
	 * @param	bInIsPlaceable					true if the class is a placeable class.
	 */
	FClassViewerNode( const FString& InClassName, const FString& InClassDisplayName );

	FClassViewerNode( const FClassViewerNode& InCopyObject);

	/**
	 * Adds the specified child to the node.
	 *
	 * @param	Child							The child to be added to this node for the tree.
	 */
	void AddChild( TSharedPtr<FClassViewerNode> Child );

	/** 
	 * Retrieves the class name this node is associated with. This is not the literal UClass name as it is missing the _C for blueprints
	 * @param	bUseDisplayName	Whether to use the display name or class name
	 */
	TSharedPtr<FString> GetClassName(bool bUseDisplayName = false) const
	{
		return bUseDisplayName ? ClassDisplayName : ClassName;
	}

	/**
	 * Retrieves the class name this node is associated with. This is not the literal UClass name as it is missing the _C for blueprints
	 * @param	NameType	Whether to use the display name or class name
	 */
	TSharedPtr<FString> GetClassName(EClassViewerNameTypeToDisplay NameType) const;

	/** Retrieves the children list. */
	TArray<TSharedPtr<FClassViewerNode>>& GetChildrenList()
	{
		return ChildrenList;
	}

	/** Checks if the class is placeable. */
	bool IsClassPlaceable() const;

	/** Checks if this is a blueprint */
	bool IsBlueprintClass() const;

	/** Checks if this is an editor module class */
	bool IsEditorOnlyClass() const;

	/** Rather this class is not allowed for the specific context */
	bool IsRestricted() const;

	/** Get the parent node for this node. */
	TSharedPtr< FClassViewerNode > GetParentNode() const;

private:
	/** The nontranslated internal name for this class. This is not necessarily the UClass's name, as that may have _C for blueprints */
	TSharedPtr<FString> ClassName;

	/** The translated display name for this class */
	TSharedPtr<FString> ClassDisplayName;

	/** List of children. */
	TArray<TSharedPtr<FClassViewerNode>> ChildrenList;

	/** Pointer to the parent to this object. */
	TWeakPtr< FClassViewerNode > ParentNode;

public:
	/** The class this node is associated with. */
	TWeakObjectPtr<UClass> Class;

	/** The blueprint this node is associated with. */
	TWeakObjectPtr<UBlueprint> Blueprint;

	/** Full object path to the class including _C, set for both blueprint and native */
	FTopLevelAssetPath ClassPath;

	/** Full object path to the parent class, may be blueprint or native */
	FTopLevelAssetPath ParentClassPath;

	/** Full path to the Blueprint that this class is loaded from, none for native classes*/
	FSoftObjectPath BlueprintAssetPath;

	/** true if the class passed the filter. */
	bool bPassesFilter;

	/** 
	 * true if the class passed all the sub-filters of the filter (regardless of the TextFilter one).
	 * This could be useful to verify e.g., that the parent class of a IsNodeAllowed() object is also valid (even though that parent will not likely pass the TextFilter).
	 */
	bool bPassesFilterRegardlessTextFilter;

	/** Data for unloaded blueprints, only valid if the class is unloaded. */
	TSharedPtr< class IUnloadedBlueprintData > UnloadedBlueprintData;

	/** The property this node will be working on. */
	TSharedPtr<class IPropertyHandle> PropertyHandle;
};
