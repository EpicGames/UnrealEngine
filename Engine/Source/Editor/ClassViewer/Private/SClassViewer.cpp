// Copyright Epic Games, Inc. All Rights Reserved.

#include "SClassViewer.h"

#include "AddToProjectConfig.h"
#include "AssetDiscoveryIndicator.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetToolsModule.h"
#include "Blueprint/BlueprintSupport.h"
#include "ClassViewerFilter.h"
#include "ClassViewerNode.h"
#include "ClassViewerProjectSettings.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDataDragDropOp.h"
#include "ContentBrowserModule.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Dialogs/Dialogs.h"
#include "DragAndDrop/ClassDragDropOp.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorClassUtils.h"
#include "EditorDirectories.h"
#include "EditorWidgetsModule.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Views/ITypedTableView.h"
#include "GameProjectGenerationModule.h"
#include "HAL/PlatformMisc.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "IDocumentation.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Layout/WidgetPath.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/MessageLog.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "Misc/FeedbackContext.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Modules/ModuleManager.h"
#include "PackageTools.h"
#include "PropertyHandle.h"
#include "SListViewSelectorDropdownMenu.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "SourceCodeNavigation.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateIconFinder.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "Trace/Detail/Channel.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/CoreRedirects.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectBase.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UnloadedBlueprintData.h"
#include "UnrealEdGlobals.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"

class FUICommandList;
class ITableRow;
class SWidget;
struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SClassViewer"

//////////////////////////////////////////////////////////////

struct FClassViewerNodeNameLess
{
	FClassViewerNodeNameLess(EClassViewerNameTypeToDisplay NameTypeToDisplay = EClassViewerNameTypeToDisplay::ClassName) : NameTypeToDisplay(NameTypeToDisplay) {}

	bool operator()(TSharedPtr<FClassViewerNode> A, TSharedPtr<FClassViewerNode> B) const
	{ 
		check(A.IsValid());
		check(B.IsValid());

		// The display name only matters when NameTypeToDisplay == DisplayName. For NameTypeToDisplay == Dynamic,
		// the class name is displayed first with the display name in parentheses, but only if it differs from the display name.
		bool bUseDisplayName = NameTypeToDisplay == EClassViewerNameTypeToDisplay::DisplayName;
		const FString& NameA = *A->GetClassName(bUseDisplayName).Get();
		const FString& NameB = *B->GetClassName(bUseDisplayName).Get();
		return NameA.Compare(NameB, ESearchCase::IgnoreCase) < 0;
	}

	EClassViewerNameTypeToDisplay NameTypeToDisplay;
};

class FClassHierarchy
{
public:
	FClassHierarchy();
	~FClassHierarchy();

	/** Populates the class hierarchy tree, pulling in all the loaded and unloaded classes. */
	void PopulateClassHierarchy();
	void PopulateClassHierarchy(const FAssetData& InAssetData) { PopulateClassHierarchy(); }

	/** Recursive function to sort a tree.
	 *	@param InOutRootNode						The current node to sort.
	 */	
	void SortChildren(TSharedPtr< FClassViewerNode >& InRootNode);

	/** Checks if a particular class is placeable.
	 *	@return The ObjectClassRoot for building a duplicate tree using.
	 */
	const TSharedPtr< FClassViewerNode > GetObjectRootNode() const
	{
		// This node should always be valid.
		check(ObjectClassRoot.IsValid())

		return ObjectClassRoot;
	}

	/** Finds the parent of a node, recursively going deeper into the hierarchy.
	 *	@param InRootNode							The current class node to examine.
	 *	@param InParentClassName					The classname to look for.
	 *	@param InParentClass						The parent class to look for.
	 *
	 *	@return The parent node.
	 */
	TSharedPtr< FClassViewerNode > FindParent(const TSharedPtr< FClassViewerNode >& InRootNode, FTopLevelAssetPath InParentClassname, const UClass* InParentClass);

	/** Updates the Class of a node. Uses the generated class package name to find the node.
	 *	@param InGeneratedClassPath		The path of the generated class to find the node for.
	 *	@param InNewClass				The class to update the node with.
	*/
	void UpdateClassInNode(FTopLevelAssetPath InGeneratedClassPath, UClass* InNewClass, UBlueprint* InNewBluePrint );

	/** Finds the node, recursively going deeper into the hierarchy. Does so by comparing generated class package names.
	 *	@param InGeneratedClassPath		The path of the generated class to find the node for.
	 *
	 *	@return The node.
	 */
	TSharedPtr< FClassViewerNode > FindNodeByGeneratedClassPath(const TSharedPtr< FClassViewerNode >& InRootNode, FTopLevelAssetPath InGeneratedClassPath);

private:
	/** Adds UClass information to The node in InOutClassPathToNode for every loaded class, creating the node if it does not exist. Does not filter classes.
	 *	@param OutRootNode						Out parameter that receives the node pointer for the root UClass.
	 *  @param InOutClassPathToNode				Map containing all nodes.
	 */	
	void CreateNodesForLoadedClasses( TSharedPtr<FClassViewerNode>& OutRootNode, TMap<FTopLevelAssetPath, TSharedPtr< FClassViewerNode >>& InOutClassPathToNode );

	/** Called when reload has finished */
	void OnReloadComplete( EReloadCompleteReason Reason );

	/** 
	 * Makes or updates a class viewer node with the data for an unloaded blueprint asset.
	 *
	 * @param InOutClassViewerNode		The node to save all the data into.
	 * @param InAssetData				The asset data to pull the tags from.
	 * @parma InClassPath				The class path
	 */
	void CreateOrUpdateUnloadedClassNode(TSharedPtr<FClassViewerNode>& InOutClassViewerNode, const FAssetData& InAssetData, FTopLevelAssetPath InClassPath);

	/**
	 * Finds the UClass and UBlueprint for the passed in node, utilizing unloaded data to find it.
	 *
	 * @param InOutClassNode		The node to find the class and fill out.
	 */
	void FindClass(TSharedPtr< FClassViewerNode > InOutClassNode);

	/** Sets the fields calculated from the in-memory UClass on the given Node, if not already set. */
	void SetClassFields(TSharedPtr<FClassViewerNode>& InOutClassNode, UClass& Class);

	/** 
	 * Recursively searches through the hierarchy to find and remove the asset. Used when deleting assets.
	 *
	 * @param InRootNode	The node to start the search with.
	 * @param InClassPath	The class path of the asset to delete
	 *
	 * @return Returns true if the asset was found and deleted successfully.
	 */
	bool FindAndRemoveNodeByClassPath(const TSharedPtr< FClassViewerNode >& InRootNode, FTopLevelAssetPath InClassPath);

	/** Callback registered to the Asset Registry to be notified when an asset is added. */
	void AddAsset(const FAssetData& InAddedAssetData);

	/** Callback registered to the Asset Registry to be notified when an asset is removed. */
	void RemoveAsset(const FAssetData& InRemovedAssetData);

	/** Apply CoreRedirects on TopLevelAssetPath. */
	void FixupClassCoreRedirects(FTopLevelAssetPath& InOutClassPath);
private:
	/** The "Object" class node that is used as a rooting point for the Class Viewer. */
	TSharedPtr< FClassViewerNode > ObjectClassRoot;

	/** Handles to various registered RequestPopulateClassHierarchy delegates */
	FDelegateHandle OnFilesLoadedRequestPopulateClassHierarchyDelegateHandle;
	FDelegateHandle OnBlueprintCompiledRequestPopulateClassHierarchyDelegateHandle;
	FDelegateHandle OnClassPackageLoadedOrUnloadedRequestPopulateClassHierarchyDelegateHandle;
};

namespace ClassViewer
{
	namespace Helpers
	{
		DECLARE_MULTICAST_DELEGATE( FPopulateClassViewer );

		/** The class hierarchy that manages the unfiltered class tree for the Class Viewer. */
		static TSharedPtr< FClassHierarchy > ClassHierarchy;

		/** Used to inform any registered Class Viewers to refresh. */
		static FPopulateClassViewer PopulateClassviewerDelegate;

		/** true if the Class Hierarchy should be populated. */
		static bool bPopulateClassHierarchy;

		// Pre-declare these functions.
		static UBlueprint* GetBlueprint( UClass* InClass );
		static void UpdateClassInNode(FTopLevelAssetPath InGeneratedClassPath, UClass* InNewClass, UBlueprint* InNewBluePrint );

		/** Util class to checks if a particular class can be made into a Blueprint, ignores deprecation
		 *
		 * @param InClass					The class to verify can be made into a Blueprint
		 * @return							TRUE if the class can be made into a Blueprint
		 */
		bool CanCreateBlueprintOfClass_IgnoreDeprecation(UClass* InClass)
		{
			// Temporarily remove the deprecated flag so we can check if it is valid for
			bool bIsClassDeprecated = InClass->HasAnyClassFlags(CLASS_Deprecated);
			InClass->ClassFlags &= ~CLASS_Deprecated;

			bool bCanCreateBlueprintOfClass = FKismetEditorUtilities::CanCreateBlueprintOfClass( InClass );

			// Reassign the deprecated flag if it was previously assigned
			if(bIsClassDeprecated)
			{
				InClass->ClassFlags |= CLASS_Deprecated;
			}

			return bCanCreateBlueprintOfClass;
		}

		/** Checks if a particular class is abstract.
		 *	@param InClass				The Class to check.
		 *	@return Returns true if the class is abstract.
		 */
		static bool IsAbstract(const UClass* InClass)
		{
			return InClass->HasAnyClassFlags(CLASS_Abstract);
		}

		/** Will create the instance of FClassHierarchy and populate the class hierarchy tree. */
		static void ConstructClassHierarchy()
		{
			if(!ClassHierarchy.IsValid())
			{
				ClassHierarchy = MakeShareable(new FClassHierarchy);

				// When created, populate the hierarchy.
				GWarn->BeginSlowTask( LOCTEXT("RebuildingClassHierarchy", "Rebuilding Class Hierarchy"), true );
				ClassHierarchy->PopulateClassHierarchy();
				GWarn->EndSlowTask();
			}
		}

		/** Cleans up the Class Hierarchy */
		static void DestroyClassHierachy()
		{
			ClassHierarchy.Reset();
		}

		/** Will populate the class hierarchy tree if previously requested. */
		static void PopulateClassHierarchy()
		{
			if(bPopulateClassHierarchy)
			{
				bPopulateClassHierarchy = false;

				GWarn->BeginSlowTask( LOCTEXT("RebuildingClassHierarchy", "Rebuilding Class Hierarchy"), true );
				ClassHierarchy->PopulateClassHierarchy();
				GWarn->EndSlowTask();
			}
		}

		/** Will enable the Class Hierarchy to be populated next Tick. */
		static void RequestPopulateClassHierarchy()
		{
			bPopulateClassHierarchy = true;
		}

		/** Refreshes all registered instances of Class Viewer/Pickers. */
		static void RefreshAll()
		{
			ClassViewer::Helpers::PopulateClassviewerDelegate.Broadcast();
		}

		/** Recursive function to build a tree, filtering out nodes based on the InitOptions and filter search terms.
		 *	@param InOutRootNode						The node that this function will add the children of to the tree.
		 *	@param InRootClassIndex						The index of the root node.
		 *	@param InClassFilter						The class filter to use to filter nodes.
		 *	@param InInitOptions						The class viewer's options, holds the AllowedClasses and DisallowedClasses.
		 *
		 *	@return Returns true if the child passed the filter.
		 */
		static bool AddChildren_Tree(TSharedPtr< FClassViewerNode >& InOutRootNode, const TSharedPtr< FClassViewerNode >& InOriginalRootNode, 
			const TSharedPtr<FClassViewerFilter>& InClassFilter, const FClassViewerInitializationOptions& InInitOptions)
		{
			bool bCheckTextFilter = true;
			InOutRootNode->bPassesFilter = InClassFilter->IsNodeAllowed(InInitOptions, InOutRootNode.ToSharedRef(), bCheckTextFilter);

			bool bReturnPassesFilter = InOutRootNode->bPassesFilter;

			bCheckTextFilter = false;
			InOutRootNode->bPassesFilterRegardlessTextFilter = bReturnPassesFilter || InClassFilter->IsNodeAllowed(InInitOptions, InOutRootNode.ToSharedRef(), bCheckTextFilter);

			TArray< TSharedPtr< FClassViewerNode > >& ChildList = InOriginalRootNode->GetChildrenList();
			for(int32 ChildIdx = 0; ChildIdx < ChildList.Num(); ChildIdx++)
			{
				TSharedPtr< FClassViewerNode > NewNode = MakeShared<FClassViewerNode>( *ChildList[ChildIdx].Get() );

				const bool bChildrenPassesFilter = AddChildren_Tree(NewNode, ChildList[ChildIdx], InClassFilter, InInitOptions);
				bReturnPassesFilter |= bChildrenPassesFilter;

				if (bChildrenPassesFilter)
				{
					InOutRootNode->AddChild(NewNode);
				}
			}

			if (bReturnPassesFilter)
			{
				InOutRootNode->GetChildrenList().Sort(FClassViewerNodeNameLess(InInitOptions.NameTypeToDisplay));
			}

			return bReturnPassesFilter;
		}

		/** Builds the class tree.
		 *	@param InOutRootNode						The node to root the tree to.
		 *	@param InClassFilter						The class filter to use to filter nodes.
		 *	@param InInitOptions						The class viewer's options, holds the AllowedClasses and DisallowedClasses.
		 *
		 *	@return A fully built tree.
		 */
		static void GetClassTree(TSharedPtr< FClassViewerNode >& InOutRootNode, const TSharedPtr<FClassViewerFilter>& InClassFilter,
			const FClassViewerInitializationOptions& InInitOptions)
		{
			const TSharedPtr< FClassViewerNode > ObjectClassRoot = ClassHierarchy->GetObjectRootNode();

			// Duplicate the node, it will have no children.
			InOutRootNode = MakeShared<FClassViewerNode>(*ObjectClassRoot);

			if (InInitOptions.bIsActorsOnly)
			{
				for (int32 ClassIdx = 0; ClassIdx < ObjectClassRoot->GetChildrenList().Num(); ClassIdx++)
				{
					TSharedPtr<FClassViewerNode> ChildNode = MakeShared<FClassViewerNode>(*ObjectClassRoot->GetChildrenList()[ClassIdx].Get());
					if (AddChildren_Tree(ChildNode, ObjectClassRoot->GetChildrenList()[ClassIdx], InClassFilter, InInitOptions))
					{
						InOutRootNode->AddChild(ChildNode);
					}
				}
			}
			else
			{
				AddChildren_Tree(InOutRootNode, ObjectClassRoot, InClassFilter, InInitOptions);
			}
		}

		/** Recursive function to build the list, filtering out nodes based on the InitOptions and filter search terms.
		 *	@param InOutRootNode						The node that this function will add the children of to the tree.
		 *	@param InRootClassIndex						The index of the root node.
		 *	@param InClassFilter						The class filter to use to filter nodes.
		 *	@param InInitOptions						The class viewer's options, holds the AllowedClasses and DisallowedClasses.
		 *
		 *	@return Returns true if the child passed the filter.
		 */
		static void AddChildren_List(TArray< TSharedPtr< FClassViewerNode > >& InOutNodeList, const TSharedPtr< FClassViewerNode >& InOriginalRootNode, 
			const TSharedPtr< FClassViewerFilter >& InClassFilter, const FClassViewerInitializationOptions& InInitOptions)
		{
			const bool bCheckTextFilter = true;
			if (InClassFilter->IsNodeAllowed(InInitOptions, InOriginalRootNode.ToSharedRef(), bCheckTextFilter))
			{
				TSharedPtr< FClassViewerNode > NewNode = MakeShared<FClassViewerNode>(*InOriginalRootNode.Get());
				NewNode->bPassesFilter = true;
				NewNode->bPassesFilterRegardlessTextFilter = true;
				NewNode->PropertyHandle = InOriginalRootNode->PropertyHandle;

				InOutNodeList.Add(NewNode);
			}

			for(const TSharedPtr<FClassViewerNode>& ChildNode : InOriginalRootNode->GetChildrenList())
			{
				FClassViewerInitializationOptions TempOptions = InInitOptions;

				// set bOnlyActors to false so that anything below Actor is added
				TempOptions.bIsActorsOnly = false;
				AddChildren_List(InOutNodeList, ChildNode, InClassFilter, InInitOptions);
			}
		}	

		/** Builds the class list.
		 *	@param InOutNodeList						The list to add all the nodes to.
		 *	@param InClassFilter						The class filter to use to filter nodes.
		 *	@param InInitOptions						The class viewer's options, holds the AllowedClasses and DisallowedClasses.
		 *
		 *	@return A fully built list.
		 */
		static void GetClassList(TArray< TSharedPtr< FClassViewerNode > >& InOutNodeList, const TSharedPtr<FClassViewerFilter>& InClassFilter, 
			const FClassViewerInitializationOptions& InInitOptions)
		{
			const TSharedPtr< FClassViewerNode > ObjectClassRoot = ClassHierarchy->GetObjectRootNode();

			// If the option to see the object root class is set, add it to the list, proceed normally from there so the actor's only filter continues to work.
			if (InInitOptions.bShowObjectRootClass)
			{
				const bool bCheckTextFilter = true;
				if (InClassFilter->IsNodeAllowed(InInitOptions, ObjectClassRoot.ToSharedRef(), bCheckTextFilter))
				{
					TSharedPtr< FClassViewerNode > NewNode = MakeShared<FClassViewerNode>(*ObjectClassRoot.Get());
					NewNode->bPassesFilter = true;
					NewNode->bPassesFilterRegardlessTextFilter = true;
					NewNode->PropertyHandle = InInitOptions.PropertyHandle;

					InOutNodeList.Add(NewNode);
				}
			}

			for(const TSharedPtr<FClassViewerNode>& ChildNode : ObjectClassRoot->GetChildrenList())
			{
				AddChildren_List(InOutNodeList, ChildNode, InClassFilter, InInitOptions);
			}
		}

		/** Retrieves the blueprint for a class index.
		 *	@param InClass							The class whose blueprint is desired.
		 *
		 *	@return									The blueprint associated with the class index.
		 */
		static UBlueprint* GetBlueprint( UClass* InClass )
		{
			if( InClass->ClassGeneratedBy && InClass->ClassGeneratedBy->IsA(UBlueprint::StaticClass()) )
			{
				return Cast<UBlueprint>(InClass->ClassGeneratedBy);
			}

			return nullptr;
		}

		/** Retrieves a few items of information on the given UClass (retrieved via the InClassIndex). 
		 *	@param InClass							The class to gather info of.
		 *	@param bInOutIsBlueprintBase			true if the class is a blueprint.
		 *	@param bInOutHasBlueprint				true if the class has a blueprint.
		 *
		 *	@return									The blueprint associated with the class index.
		 */
		static void GetClassInfo( TWeakObjectPtr<UClass> InClass, bool& bInOutIsBlueprintBase, bool& bInOutHasBlueprint )
		{
			if (UClass* Class = InClass.Get())
			{
				bInOutIsBlueprintBase = CanCreateBlueprintOfClass_IgnoreDeprecation( Class );
				bInOutHasBlueprint = Class->ClassGeneratedBy != nullptr;
			}
			else
			{
				bInOutIsBlueprintBase = false;
				bInOutHasBlueprint = false;
			}
		}

		/**
		 * Creates a blueprint from a class.
		 *
		 * @param	InCreationClass			The class to create the blueprint from.
		 * @param	InParentContent			The content to parent the STextEntryPopup to. 
		 */
		static void CreateBlueprint(const FString& InBlueprintName, UClass* InCreationClass)
		{
			if(InCreationClass == nullptr || !FKismetEditorUtilities::CanCreateBlueprintOfClass(InCreationClass))
			{
				FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "InvalidClassToMakeBlueprintFrom", "Invalid class to make a Blueprint of."));
				return;
			}

			// Get the full name of where we want to create the physics asset.
			FString PackageName = InBlueprintName;

			// Then find/create it.
			UPackage* Package = CreatePackage( *PackageName);
			check(Package);

			// Handle fully loading packages before creating new objects.
			TArray<UPackage*> TopLevelPackages;
			TopLevelPackages.Add( Package->GetOutermost() );
			if( !UPackageTools::HandleFullyLoadingPackages( TopLevelPackages, NSLOCTEXT("UnrealEd", "CreateANewObject", "Create a new object") ) )
			{
				// Can't load package
				return;
			}

			FName BPName(*FPackageName::GetLongPackageAssetName(PackageName));

			if(PromptUserIfExistingObject(BPName.ToString(), PackageName, Package))
			{
				// Create and init a new Blueprint
				UBlueprint* NewBP = FKismetEditorUtilities::CreateBlueprint(InCreationClass, Package, BPName, BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass(), FName("ClassViewer"));
				if(NewBP)
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewBP);

					// Notify the asset registry
					FAssetRegistryModule::AssetCreated(NewBP);

					// Mark the package dirty...
					Package->MarkPackageDirty();
				}
			}

			// All viewers must refresh.
			RefreshAll();
		}

		/**
		 * Creates a SaveAssetDialog for specifying the path for the new blueprint
		 */
		static void OpenCreateBlueprintDialog(UClass* InCreationClass)
		{
			// Determine default path for the Save Asset dialog
			FString DefaultPath;
			const FString DefaultDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::NEW_ASSET);
			FPackageName::TryConvertFilenameToLongPackageName(DefaultDirectory, DefaultPath);

			if (DefaultPath.IsEmpty())
			{
				DefaultPath = TEXT("/Game/Blueprints");
			}

			// Determine default filename for the Save Asset dialog
			check(InCreationClass != nullptr);
			const FString ClassName = InCreationClass->ClassGeneratedBy ? InCreationClass->ClassGeneratedBy->GetName() : InCreationClass->GetName();
			FString DefaultName = LOCTEXT("PrefixNew", "New").ToString() + ClassName;

			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
			FString UniquePackageName;
			FString UniqueAssetName;
			AssetToolsModule.Get().CreateUniqueAssetName(DefaultPath / DefaultName, TEXT(""), UniquePackageName, UniqueAssetName);
			DefaultName = FPaths::GetCleanFilename(UniqueAssetName);

			// Initialize SaveAssetDialog config
			FSaveAssetDialogConfig SaveAssetDialogConfig;
			SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("CreateBlueprintDialogTitle", "Create Blueprint Class");
			SaveAssetDialogConfig.DefaultPath = DefaultPath;
			SaveAssetDialogConfig.DefaultAssetName = DefaultName;
			SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;

			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
			if (!SaveObjectPath.IsEmpty())
			{
				const FString PackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
				const FString PackageFilename = FPackageName::LongPackageNameToFilename(PackageName);
				const FString PackagePath = FPaths::GetPath(PackageFilename);

				CreateBlueprint(PackageName, InCreationClass);
				FEditorDirectories::Get().SetLastDirectory(ELastDirectory::NEW_ASSET, PackagePath);
			}
		}

		/** Returns the tooltip to display when attempting to derive a Blueprint */
		FText GetCreateBlueprintTooltip(UClass* InCreationClass)
		{
			if(InCreationClass->HasAnyClassFlags(CLASS_Deprecated))
			{
				return LOCTEXT("ClassViewerMenuCreateDeprecatedBlueprint_Tooltip", "Class is deprecated!");
			}
			else
			{
				return LOCTEXT("ClassViewerMenuCreateBlueprint_Tooltip", "Creates a Blueprint Class using this class as a base.");
			}
		}

		/** Returns TRUE if you can derive a Blueprint */
		bool CanOpenCreateBlueprintDialog(UClass* InCreationClass)
		{
			return !InCreationClass->HasAnyClassFlags(CLASS_Deprecated);
		}

		/**
		 * Creates a class wizard for creating a new C++ class
		 *
		 * @param	InParentContent			The content to parent the STextEntryPopup to. 
		 */
		static void OpenCreateCPlusPlusClassWizard(UClass* InCreationClass)
		{
			FGameProjectGenerationModule::Get().OpenAddCodeToProjectDialog(
				FAddToProjectConfig()
				.ParentClass(InCreationClass)
				.ParentWindow(FGlobalTabmanager::Get()->GetRootWindow())
			);
		}

		/**
		 * Creates a blueprint from a class.
		 *
		 * @param	InOutClassNode			Class node to pull what class to load and to update information in.
		 */
		static void LoadClass(TSharedPtr< FClassViewerNode > InOutClassNode)
		{
			GWarn->BeginSlowTask(LOCTEXT("LoadPackage", "Loading Package..."), true);
			UClass* Class = LoadObject<UClass>(nullptr, *InOutClassNode->ClassPath.ToString());
			GWarn->EndSlowTask();

			if (Class)
			{
				InOutClassNode->Blueprint = Cast<UBlueprint>(Class->ClassGeneratedBy);
				InOutClassNode->Class = Class;

				// Tell the original node to update so when a refresh happens it will still know about the newly loaded class.
				ClassViewer::Helpers::UpdateClassInNode(InOutClassNode->ClassPath, InOutClassNode->Class.Get(), InOutClassNode->Blueprint.Get() );
			}
			else
			{
				FMessageLog EditorErrors("EditorErrors");
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("ObjectName"), FText::FromString(InOutClassNode->ClassPath.ToString()));
				EditorErrors.Error(FText::Format(LOCTEXT("PackageLoadFail", "Failed to load class {ObjectName}"), Arguments));
			}
		}

		/**
		 * Opens a blueprint.
		 *
		 * @param	InBlueprint			The blueprint to open.
		 */
		static void OpenBlueprintTool(UBlueprint* InBlueprint)
		{
			if( InBlueprint != nullptr )
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(InBlueprint);
			}
		}

		/**
		 * Opens a class's source file.
		 *
		 * @param	InClass		The class to open source for.
		 */
		static void OpenClassInIDE(UClass* InClass)
		{
			//ignore result
			FSourceCodeNavigation::NavigateToClass(InClass);
		}

		/**
		 * Finds the blueprint or class in the content browser. Blueprint prioritized because if there is a blueprint we want to find that.
		 *
		 * @param	InBlueprint			The blueprint to find.
		 * @param	InClass				The class to find.
		 */
		static void FindInContentBrowser(UBlueprint* InBlueprint, UClass* InClass)
		{
			// If there is a blueprint, use the blueprint instead of the class. Otherwise it will not fully find the requested object.
			if(InBlueprint)
			{
				TArray<UObject*> Objects;
				Objects.Add(InBlueprint);
				GEditor->SyncBrowserToObjects(Objects);
			}
			else if (InClass)
			{
				TArray<UObject*> Objects;
				Objects.Add(InClass);
				GEditor->SyncBrowserToObjects(Objects);
			}
		}

		/** Updates the Class of a node. Uses the generated class package name to find the node.
		*	@param InGeneratedClassPath			The name of the generated class to find the node for.
		*	@param InNewClass					The class to update the node with.
		*/
		static void UpdateClassInNode(FTopLevelAssetPath InGeneratedClassPath, UClass* InNewClass, UBlueprint* InNewBluePrint )
		{
			ClassHierarchy->UpdateClassInNode(InGeneratedClassPath, InNewClass, InNewBluePrint );
		}

		static TSharedRef<SWidget> CreateMenu(UClass* Class, const bool bIsBlueprint, const bool bHasBlueprint)
		{
			// Empty list of commands.
			TSharedPtr< FUICommandList > Commands;

			const bool bShouldCloseWindowAfterMenuSelection = true;	// Set the menu to automatically close when the user commits to a choice
			FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, Commands);
			{
				if (bIsBlueprint)
				{
					TAttribute<FText>::FGetter DynamicTooltipGetter;
					DynamicTooltipGetter.BindStatic(&ClassViewer::Helpers::GetCreateBlueprintTooltip, Class);
					TAttribute<FText> DynamicTooltipAttribute = TAttribute<FText>::Create(DynamicTooltipGetter);

					MenuBuilder.AddMenuEntry(
						LOCTEXT("ClassViewerMenuCreateBlueprint", "Create Blueprint Class..."),
						DynamicTooltipAttribute,
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateStatic(&ClassViewer::Helpers::OpenCreateBlueprintDialog, Class),
							FCanExecuteAction::CreateStatic(&ClassViewer::Helpers::CanOpenCreateBlueprintDialog, Class)
							)
						);
				}

				if (bHasBlueprint)
				{
					MenuBuilder.BeginSection("ClassViewerDropDownHasBlueprint");
					{
						FUIAction Action(FExecuteAction::CreateStatic(&ClassViewer::Helpers::OpenBlueprintTool, ClassViewer::Helpers::GetBlueprint(Class)));
						MenuBuilder.AddMenuEntry(LOCTEXT("ClassViewerMenuEditBlueprint", "Edit Blueprint Class..."), LOCTEXT("ClassViewerMenuEditBlueprint_Tooltip", "Open the Blueprint Class in the editor."), FSlateIcon(), Action);
					}
					MenuBuilder.EndSection();

					MenuBuilder.BeginSection("ClassViewerDropDownHasBlueprint2");
					{
						FUIAction Action(FExecuteAction::CreateStatic(&ClassViewer::Helpers::FindInContentBrowser, ClassViewer::Helpers::GetBlueprint(Class), Class));
						MenuBuilder.AddMenuEntry(LOCTEXT("ClassViewerMenuFindContent", "Find in Content Browser..."), LOCTEXT("ClassViewerMenuFindContent_Tooltip", "Find in Content Browser"), FSlateIcon(), Action);
					}
					MenuBuilder.EndSection();
				}
				else
				{
					MenuBuilder.BeginSection("ClassViewerIsCode");
					{
						FUIAction Action(FExecuteAction::CreateStatic(&ClassViewer::Helpers::OpenClassInIDE, Class));
						MenuBuilder.AddMenuEntry(LOCTEXT("ClassViewerMenuOpenCPlusPlusClass", "Open Source Code..."), LOCTEXT("ClassViewerMenuOpenCPlusPlusClass_Tooltip", "Open the source file for this class in the IDE."), FSlateIcon(), Action);
					}
					{
						FUIAction Action(FExecuteAction::CreateStatic(&ClassViewer::Helpers::OpenCreateCPlusPlusClassWizard, Class));
						MenuBuilder.AddMenuEntry(LOCTEXT("ClassViewerMenuCreateCPlusPlusClass", "Create New C++ Class..."), LOCTEXT("ClassViewerMenuCreateCPlusPlusClass_Tooltip", "Creates a new C++ class using this class as a base."), FSlateIcon(), Action);
					}
					MenuBuilder.EndSection();
				}
			}

			return MenuBuilder.MakeWidget();
		}


	} // namespace Helpers
} // namespace ClassViewer

/** Delegate used with the Class Viewer in 'class picking' mode.  You'll bind a delegate when the
	class viewer widget is created, which will be fired off when the selected class is double clicked */
DECLARE_DELEGATE_OneParam( FOnClassItemDoubleClickDelegate, TSharedPtr<FClassViewerNode> );

/** The item used for visualizing the class in the tree. */
class SClassItem : public STableRow< TSharedPtr<FString> >
{
public:
	
	SLATE_BEGIN_ARGS( SClassItem )
		: _ClassName()
		, _bIsPlaceable(false)
		, _bIsInClassViewer( true )
		, _bDynamicClassLoading( true )
		, _HighlightText()
		, _Font(FAppStyle::Get().GetFontStyle("NormalFont"))
		{}

		/** The classname this item contains. */
		SLATE_ARGUMENT( TSharedPtr<FString>, ClassName )
		/** true if this item is a placeable object. */
		SLATE_ARGUMENT( bool, bIsPlaceable )
		/** true if this item is in a Class Viewer (as opposed to a Class Picker) */
		SLATE_ARGUMENT( bool, bIsInClassViewer )
		/** true if this item should allow dynamic class loading */
		SLATE_ARGUMENT( bool, bDynamicClassLoading )
		/** The text this item should highlight, if any. */
		SLATE_ARGUMENT( FText, HighlightText )
		/** The font this item will use. */
		SLATE_ARGUMENT( FSlateFontInfo, Font )
		/** The node this item is associated with. */
		SLATE_ARGUMENT( TSharedPtr<FClassViewerNode>, AssociatedNode)
		/** the delegate for handling double clicks outside of the SClassItem */
		SLATE_ARGUMENT( FOnClassItemDoubleClickDelegate, OnClassItemDoubleClicked )
		/** On Class Picked callback. */
		SLATE_EVENT( FOnDragDetected, OnDragDetected )

	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		ClassName = InArgs._ClassName;
		bIsClassPlaceable = InArgs._bIsPlaceable;
		bIsInClassViewer = InArgs._bIsInClassViewer;
		bDynamicClassLoading = InArgs._bDynamicClassLoading;
		AssociatedNode = InArgs._AssociatedNode;
		OnDoubleClicked = InArgs._OnClassItemDoubleClicked;
		
		bool bIsBlueprint(false);
		bool bHasBlueprint(false);

		//SetEnabled( AssociatedNode->Restrictions.Num() == 0 );

		ClassViewer::Helpers::GetClassInfo(AssociatedNode->Class, bIsBlueprint, bHasBlueprint);
		
		struct Local
		{

			static TSharedPtr<SToolTip> GetToolTip(TSharedPtr<FClassViewerNode> AssociatedNode)
			{
				TSharedPtr<SToolTip> ToolTip;
				if( AssociatedNode->PropertyHandle.IsValid() && AssociatedNode->IsRestricted() )
				{
					FText RestrictionToolTip;
					AssociatedNode->PropertyHandle->GenerateRestrictionToolTip(*AssociatedNode->GetClassName(),RestrictionToolTip);

					ToolTip = IDocumentation::Get()->CreateToolTip(RestrictionToolTip, nullptr, "", "");
				}
				else if (UClass* Class = AssociatedNode->Class.Get())
				{
					UPackage*  Package  = Class->GetOutermost();
					ToolTip = FEditorClassUtils::GetTooltip(Class);
				}
				else if (!AssociatedNode->ClassPath.IsNull())
				{
					ToolTip = SNew(SToolTip).Text(FText::FromString(AssociatedNode->ClassPath.ToString()));
				}

				return ToolTip;
			}
		};

		bool bIsRestricted = AssociatedNode->IsRestricted();

		const FSlateBrush* ClassIcon = FSlateIconFinder::FindIconBrushForClass(AssociatedNode->Class.Get());
		
		this->ChildSlot
		[
			SNew(SHorizontalBox)
				
			+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew( SExpanderArrow, SharedThis(this) )
				]

			+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding( 0.0f, 2.0f, 6.0f, 2.0f )
				[
					SNew( SImage )
					.Image(	ClassIcon )
					.Visibility( ClassIcon != FAppStyle::GetDefaultBrush()? EVisibility::Visible : EVisibility::Collapsed )
				]

			+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding( 0.0f, 3.0f, 6.0f, 3.0f )
				.VAlign(VAlign_Center)
				[
					SNew( STextBlock )
						.Text( FText::FromString(*ClassName.Get()) )
						.Font(InArgs._Font)
						.HighlightText(InArgs._HighlightText)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.ToolTip(Local::GetToolTip(AssociatedNode))
						.IsEnabled(!bIsRestricted)
				]

			+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding( 0.0f, 1.0f, 6.0f, 1.0f )
				[
					SNew( SComboButton )
						.ContentPadding(FMargin(2.0f))
						.Visibility(this, &SClassItem::ShowOptions)
						.OnGetMenuContent(this, &SClassItem::GenerateDropDown)
				]
		];

		UE_LOGF(LogEditorClassViewer, VeryVerbose, "CLASS [%ls]", **ClassName);


		STableRow< TSharedPtr<FString> >::ConstructInternal(
			STableRow::FArguments()
				.ShowSelection(true)
				.OnDragDetected(InArgs._OnDragDetected),
			InOwnerTableView
		);	
	}

private:
	virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override
	{
		// If in a Class Viewer and it has not been loaded, load the class when double-left clicking.
		if ( bIsInClassViewer )
		{
			if( bDynamicClassLoading && AssociatedNode->Class == nullptr && AssociatedNode->UnloadedBlueprintData.IsValid() && InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
			{
				ClassViewer::Helpers::LoadClass(AssociatedNode);
			}
			// If there is a blueprint, open it. Otherwise try to open the class header.
			if(AssociatedNode->Blueprint.IsValid())
			{
				ClassViewer::Helpers::OpenBlueprintTool(AssociatedNode->Blueprint.Get());
			}
			else
			{
				ClassViewer::Helpers::OpenClassInIDE(AssociatedNode->Class.Get());
			}
		}
		else
		{
			OnDoubleClicked.ExecuteIfBound( AssociatedNode );
		}
		return FReply::Handled();
	}

	EVisibility ShowOptions() const
	{
		// If it's in viewer mode, show the options combo button.
		if(bIsInClassViewer)
		{
			bool bIsBlueprint(false);
			bool bHasBlueprint(false);

			ClassViewer::Helpers::GetClassInfo(AssociatedNode->Class, bIsBlueprint, bHasBlueprint);

			return (bIsBlueprint || AssociatedNode->Blueprint.IsValid())? EVisibility::Visible : EVisibility::Collapsed;
		}

		return EVisibility::Collapsed;
	}

	/**
	 * Generates the drop down menu for the item.
	 *
	 * @return		The drop down menu widget.
	 */
	TSharedRef<SWidget> GenerateDropDown()
	{
		if (UClass* Class = AssociatedNode->Class.Get())
		{
			bool bIsBlueprint(false);
			bool bHasBlueprint(false);

			ClassViewer::Helpers::GetClassInfo(Class, bIsBlueprint, bHasBlueprint);
			bHasBlueprint = AssociatedNode->Blueprint.IsValid();
			return ClassViewer::Helpers::CreateMenu(Class, bIsBlueprint, bHasBlueprint);
		}

		return SNullWidget::NullWidget;
	}

private:

	/** The class name for which this item is associated with. */
	TSharedPtr<FString> ClassName;

	/** true if this class is placeable. */
	bool bIsClassPlaceable;

	/** true if in a Class Viewer (as opposed to a Class Picker). */
	bool bIsInClassViewer;

	/** true if dynamic class loading is permitted. */
	bool bDynamicClassLoading;

	/** The Class Viewer Node this item is associated with. */
	TSharedPtr< FClassViewerNode > AssociatedNode;

	/** the on Double Clicked delegate */
	FOnClassItemDoubleClickDelegate OnDoubleClicked;
};

static void OnModulesChanged(FName ModuleThatChanged, EModuleChangeReason ReasonForChange)
{
	ClassViewer::Helpers::RequestPopulateClassHierarchy();
}

FClassHierarchy::FClassHierarchy()
{
	// Register with the Asset Registry to be informed when it is done loading up files.
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
	OnFilesLoadedRequestPopulateClassHierarchyDelegateHandle = AssetRegistry.OnFilesLoaded().AddStatic( ClassViewer::Helpers::RequestPopulateClassHierarchy );
	AssetRegistry.OnAssetAdded().AddRaw( this, &FClassHierarchy::AddAsset);
	AssetRegistry.OnAssetRemoved().AddRaw( this, &FClassHierarchy::RemoveAsset );

	// Register to have Populate called when doing a Reload.
	FCoreUObjectDelegates::ReloadCompleteDelegate.AddRaw( this, &FClassHierarchy::OnReloadComplete );

	// Register to have Populate called when a Blueprint is compiled.
	OnBlueprintCompiledRequestPopulateClassHierarchyDelegateHandle            = GEditor->OnBlueprintCompiled().AddStatic(ClassViewer::Helpers::RequestPopulateClassHierarchy);
	OnClassPackageLoadedOrUnloadedRequestPopulateClassHierarchyDelegateHandle = GEditor->OnClassPackageLoadedOrUnloaded().AddStatic(ClassViewer::Helpers::RequestPopulateClassHierarchy);

	FModuleManager::Get().OnModulesChanged().AddStatic(&OnModulesChanged);
}

FClassHierarchy::~FClassHierarchy()
{
	// Unregister with the Asset Registry to be informed when it is done loading up files.
	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::Get().GetModulePtr<FAssetRegistryModule>(AssetRegistryConstants::ModuleName))
	{
		IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnFilesLoaded().Remove(OnFilesLoadedRequestPopulateClassHierarchyDelegateHandle);
			AssetRegistry->OnAssetAdded().RemoveAll(this);
			AssetRegistry->OnAssetRemoved().RemoveAll(this);
		}

		// Unregister to have Populate called when doing a Reload.
		FCoreUObjectDelegates::ReloadCompleteDelegate.RemoveAll(this);

		if (GEditor)
		{
			// Unregister to have Populate called when a Blueprint is compiled.
			GEditor->OnBlueprintCompiled().Remove(OnBlueprintCompiledRequestPopulateClassHierarchyDelegateHandle);
			GEditor->OnClassPackageLoadedOrUnloaded().Remove(OnClassPackageLoadedOrUnloadedRequestPopulateClassHierarchyDelegateHandle);
		}
	}

	FModuleManager::Get().OnModulesChanged().RemoveAll(this);
}

void FClassHierarchy::OnReloadComplete(EReloadCompleteReason Reason)
{
	ClassViewer::Helpers::RequestPopulateClassHierarchy();
}

void FClassHierarchy::CreateNodesForLoadedClasses(TSharedPtr<FClassViewerNode>& OutRootNode, TMap<FTopLevelAssetPath, TSharedPtr< FClassViewerNode >>& InOutClassPathToNode)
{
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* CurrentClass = *ClassIt;
		// Ignore deprecated and temporary trash classes.
		if (CurrentClass->HasAnyClassFlags(CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Hidden) ||
			FBlueprintSupport::IsClassPlaceholder(CurrentClass) ||
			FKismetEditorUtilities::IsClassABlueprintSkeleton(CurrentClass))
		{
			continue;
		}
		TSharedPtr<FClassViewerNode>& Node = InOutClassPathToNode.FindOrAdd(CurrentClass->GetClassPathName());
		if (!Node)
		{
			Node = MakeShared<FClassViewerNode>(CurrentClass->GetName(), CurrentClass->GetDisplayNameText().ToString());
		}
		SetClassFields(Node, *CurrentClass);
	}
	TSharedPtr<FClassViewerNode>* ExistingRoot = InOutClassPathToNode.Find(UObject::StaticClass()->GetClassPathName());
	check(ExistingRoot && ExistingRoot->IsValid());
	OutRootNode = *ExistingRoot;
}

TSharedPtr< FClassViewerNode > FClassHierarchy::FindParent(const TSharedPtr< FClassViewerNode >& InRootNode, FTopLevelAssetPath InParentClassname, const UClass* InParentClass)
{
	// Check if the current node is the parent classname that is being searched for.
	if(InRootNode->ClassPath == InParentClassname)
	{
		// Return the node if it is the correct parent, this ends the recursion.
		return InRootNode;
	}
	else
	{
		// If a class does not have a generated classname, we look up the parent class and compare.
		const UClass* ParentClass = InParentClass;

		if(const UClass* RootClass = InRootNode->Class.Get())
		{
			if(ParentClass == RootClass)
			{
				return InRootNode;
			}
		}

	}

	TSharedPtr< FClassViewerNode > ReturnNode;

	// Search the children recursively, one of them might have the parent.
	for(int32 ChildClassIndex = 0; !ReturnNode.IsValid() && ChildClassIndex < InRootNode->GetChildrenList().Num(); ChildClassIndex++)
	{
		// Check the child, then check the return to see if it is valid. If it is valid, end the recursion.
		ReturnNode = FindParent(InRootNode->GetChildrenList()[ChildClassIndex], InParentClassname, InParentClass);

		if(ReturnNode.IsValid())
		{
			break;
		}
	}

	return ReturnNode;
}

TSharedPtr< FClassViewerNode > FClassHierarchy::FindNodeByGeneratedClassPath(const TSharedPtr< FClassViewerNode >& InRootNode, FTopLevelAssetPath InGeneratedClassPath)
{
	if(InRootNode->ClassPath == InGeneratedClassPath)
	{
		return InRootNode;
	}

	TSharedPtr< FClassViewerNode > ReturnNode;

	// Search the children recursively, one of them might have the parent.
	for(int32 ChildClassIndex = 0; !ReturnNode.IsValid() && ChildClassIndex < InRootNode->GetChildrenList().Num(); ChildClassIndex++)
	{
		// Check the child, then check the return to see if it is valid. If it is valid, end the recursion.
		ReturnNode = FindNodeByGeneratedClassPath(InRootNode->GetChildrenList()[ChildClassIndex], InGeneratedClassPath);

		if(ReturnNode.IsValid())
		{
			break;
		}
	}

	return ReturnNode;
}

void FClassHierarchy::UpdateClassInNode(FTopLevelAssetPath InGeneratedClassPath, UClass* InNewClass, UBlueprint* InNewBluePrint )
{
	TSharedPtr< FClassViewerNode > Node = FindNodeByGeneratedClassPath(ObjectClassRoot, InGeneratedClassPath);

	if( Node.IsValid() )
	{
		Node->Class = InNewClass;
		Node->Blueprint = InNewBluePrint;
	}
}

bool FClassHierarchy::FindAndRemoveNodeByClassPath(const TSharedPtr< FClassViewerNode >& InRootNode, FTopLevelAssetPath InClassPath)
{
	FixupClassCoreRedirects(InClassPath);

	TFunction<bool(const TSharedPtr<FClassViewerNode>&)> FindAndRemoveNodeRecursive;
	FindAndRemoveNodeRecursive = [&InClassPath, &FindAndRemoveNodeRecursive](const TSharedPtr<FClassViewerNode>& InRootNode)
		{
			// Search the children recursively, one of them might have the parent.
			TArray<TSharedPtr<FClassViewerNode>>& ChildrenList = InRootNode->GetChildrenList();
			for (int32 ChildClassIndex = 0; ChildClassIndex < ChildrenList.Num(); ChildClassIndex++)
			{
				if (ChildrenList[ChildClassIndex]->ClassPath == InClassPath)
				{
					ChildrenList.RemoveAt(ChildClassIndex);
					return true;
				}

				// Check the child, then check the return to see if it is valid. If it is valid, end the recursion.
				if (FindAndRemoveNodeRecursive(ChildrenList[ChildClassIndex]))
				{
					return true;
				}
			}

			return false;
		};

	return FindAndRemoveNodeRecursive(InRootNode);
}

void FClassHierarchy::RemoveAsset(const FAssetData& InRemovedAssetData)
{
	// BPGCs can be missing if it was already deleted prior to the notification being sent. 
	// Let's try to reconstruct the generated class path from the BP object path.
	bool bGenerateClassPathIfMissing = true;
	const FTopLevelAssetPath ClassPath = FEditorClassUtils::GetClassPathNameFromAsset(InRemovedAssetData, bGenerateClassPathIfMissing);

	if (!ClassPath.IsNull() && FindAndRemoveNodeByClassPath(ObjectClassRoot, ClassPath))
	{
		// All viewers must refresh.
		ClassViewer::Helpers::RefreshAll();
	}
}

void FClassHierarchy::AddAsset(const FAssetData& InAddedAssetData)
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
	if (AssetRegistry.IsLoadingAssets())
	{
		return;
	}

	const FTopLevelAssetPath ClassPath = FEditorClassUtils::GetClassPathNameFromAsset(InAddedAssetData);

	// Make sure that the node does not already exist. There is a bit of double adding going on at times and this prevents it.
	if (!ClassPath.IsNull() && !FindNodeByGeneratedClassPath(ObjectClassRoot, ClassPath).IsValid())
	{
		TSharedPtr<FClassViewerNode> NewNode;
		CreateOrUpdateUnloadedClassNode(NewNode, InAddedAssetData, ClassPath);

		// Find the blueprint if it's loaded.
		FindClass(NewNode);

		// Resolve the parent's class name locally and use it to find the parent's class.
		FString ParentClassPath = NewNode->ParentClassPath.ToString();
		UClass* ParentClass = FindObject<UClass>(nullptr, *ParentClassPath);
		TSharedPtr< FClassViewerNode > ParentNode = FindParent(ObjectClassRoot, NewNode->ParentClassPath, ParentClass); 
		if (ParentNode.IsValid())
		{
			ParentNode->AddChild(NewNode);

			// Make sure the children are properly sorted.
			SortChildren(ObjectClassRoot);

			// All Viewers must repopulate.
			ClassViewer::Helpers::RefreshAll();
		}
	}
}

void FClassHierarchy::SortChildren( TSharedPtr< FClassViewerNode >& InRootNode)
{
	TArray< TSharedPtr< FClassViewerNode > >& ChildList = InRootNode->GetChildrenList();
	for(int32 ChildIndex = 0; ChildIndex < ChildList.Num(); ChildIndex++)
	{
		// Check the child, then check the return to see if it is valid. If it is valid, end the recursion.
		SortChildren(ChildList[ChildIndex]);
	}

	// Sort the children.
	ChildList.Sort(FClassViewerNodeNameLess());
}

void FClassHierarchy::FindClass(TSharedPtr<FClassViewerNode> InOutClassNode)
{
	UClass* Class = FindObject<UClass>(nullptr, *InOutClassNode->ClassPath.ToString());

	if (Class)
	{
		SetClassFields(InOutClassNode, *Class);
	}
}

void FClassHierarchy::SetClassFields(TSharedPtr<FClassViewerNode>& InOutClassNode, UClass& Class)
{
	if (InOutClassNode->Class.Get())
	{
		// Already set
		return;
	}

	// Fields that can als be set from FAssetData
	if (InOutClassNode->ClassPath.IsNull())
	{
		InOutClassNode->ClassPath = Class.GetClassPathName();
	}
	if (InOutClassNode->ParentClassPath.IsNull())
	{
		if (Class.GetSuperClass())
		{
			InOutClassNode->ParentClassPath = Class.GetSuperClass()->GetClassPathName();
		}
	}

	// UClass-specific fields
	InOutClassNode->Blueprint = Cast<UBlueprint>(Class.ClassGeneratedBy);
	InOutClassNode->Class = &Class;
}


void FClassHierarchy::FixupClassCoreRedirects(FTopLevelAssetPath& InOutClassPath)
{
	const FCoreRedirectObjectName OldName = FCoreRedirectObjectName(InOutClassPath);
	const FCoreRedirectObjectName NewName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, OldName);
	
	if (OldName != NewName)
	{
		const FString OldClassPath = InOutClassPath.ToString();

		// Only do the fixup if the old class name isn't in memory.
		const UClass* FoundOldClass = UClass::TryFindTypeSlow<UClass>(OldClassPath);
		const FTopLevelAssetPath NewClassPath(NewName.ToString());

		if (!FoundOldClass || FoundOldClass->GetClassPathName() == NewClassPath)
		{
			InOutClassPath = NewClassPath;
		}
	}
}


void FClassHierarchy::CreateOrUpdateUnloadedClassNode(TSharedPtr<FClassViewerNode>& InOutClassViewerNode, const FAssetData& InAssetData, FTopLevelAssetPath InClassPath)
{
	if (!InOutClassViewerNode)
	{
		const FString ClassName = InAssetData.AssetName.ToString();
		FString ClassDisplayName = InAssetData.GetTagValueRef<FString>(FBlueprintTags::BlueprintDisplayName);
		if (ClassDisplayName.IsEmpty())
		{
			ClassDisplayName = ClassName;
		}
		InOutClassViewerNode = MakeShared<FClassViewerNode>(ClassName, ClassDisplayName);
	}

	if (InOutClassViewerNode->UnloadedBlueprintData.IsValid())
	{
		// Already set
		return;
	}

	// Fields that can also be set from UClass*

	InOutClassViewerNode->ClassPath = InClassPath;

	if (InOutClassViewerNode->ParentClassPath.IsNull())
	{
		FString ParentClassPathString;
		if (InAssetData.GetTagValue(FBlueprintTags::ParentClassPath, ParentClassPathString))
		{
			FTopLevelAssetPath ParentClassPath(FTopLevelAssetPath(*FPackageName::ExportTextPathToObjectPath(ParentClassPathString)));
			FixupClassCoreRedirects(ParentClassPath);
			InOutClassViewerNode->ParentClassPath = ParentClassPath;
		}
	}

	// Blueprint-specific fields

	InOutClassViewerNode->BlueprintAssetPath = InAssetData.GetSoftObjectPath();

	// It is an unloaded blueprint, so we need to create the structure that will hold the data.
	TSharedPtr<FUnloadedBlueprintData> UnloadedBlueprintData = MakeShareable(new FUnloadedBlueprintData(InOutClassViewerNode));
	InOutClassViewerNode->UnloadedBlueprintData = UnloadedBlueprintData;

	const bool bNormalBlueprintType = InAssetData.GetTagValueRef<FString>(FBlueprintTags::BlueprintType) == TEXT("BPType_Normal");
	InOutClassViewerNode->UnloadedBlueprintData->SetNormalBlueprintType(bNormalBlueprintType);

	// Get the class flags.
	const uint32 ClassFlags = InAssetData.GetTagValueRef<uint32>(FBlueprintTags::ClassFlags);
	InOutClassViewerNode->UnloadedBlueprintData->SetClassFlags(ClassFlags);

	// Get interface class paths.
	TArray<FString> ImplementedInterfaces;
	FEditorClassUtils::GetImplementedInterfaceClassPathsFromAsset(InAssetData, ImplementedInterfaces);
	for (const FString& InterfacePath : ImplementedInterfaces)
	{
		UnloadedBlueprintData->AddImplementedInterface(InterfacePath);
	}
}

void FClassHierarchy::PopulateClassHierarchy()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FClassHierarchy::PopulateClassHierarchy);

	// Fetch all classes from AssetRegistry blueprint data (which covers unloaded classes), and in-memory UClasses.
	// Create a node for each one with unioned data from the AssetRegistry or UClass for that class.
	// Set parent/child pointers to create a tree, and store this tree in this->ObjectClassRoot
	TMap<FTopLevelAssetPath, TSharedPtr<FClassViewerNode>> ClassPathToNode;

	// Create a node for every Blueprint class listed in the AssetRegistry and set the Blueprint fields
	// Retrieve all blueprint classes 
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
		FString ClassPathString;

		TArray<FAssetData> Assets;
		AssetRegistry.GetAssetsByClass(UBlueprint::StaticClass()->GetClassPathName(), Assets, /*bSearchSubClasses=*/true);

		for (const FAssetData& AssetData : Assets)
		{
			FTopLevelAssetPath ClassPath = FEditorClassUtils::GetClassPathNameFromAssetTag(AssetData);
			if (!ClassPath.IsNull())
			{
				FixupClassCoreRedirects(ClassPath);
				TSharedPtr<FClassViewerNode>& Node = ClassPathToNode.FindOrAdd(ClassPath);
				CreateOrUpdateUnloadedClassNode(Node, AssetData, ClassPath);
			}
			else
			{
				UE_LOGF(LogEditorClassViewer, Warning, "AssetRegistry Blueprint %ls is missing tag value for %ls. Blueprint will not be available to ClassViewer when unloaded.",
					*AssetData.GetObjectPathString(), *FBlueprintTags::GeneratedClassPath.ToString());
			}
		}

		Assets.Reset();
		AssetRegistry.GetAssetsByClass(UBlueprintGeneratedClass::StaticClass()->GetClassPathName(), Assets, /*bSearchSubClasses=*/true);

		for (const FAssetData& AssetData : Assets)
		{
			FTopLevelAssetPath ClassPathNameFromAssetPath = AssetData.GetSoftObjectPath().GetAssetPath();
			FixupClassCoreRedirects(ClassPathNameFromAssetPath);
			TSharedPtr<FClassViewerNode>& Node = ClassPathToNode.FindOrAdd(ClassPathNameFromAssetPath);
			CreateOrUpdateUnloadedClassNode(Node, AssetData, ClassPathNameFromAssetPath);
		}
	}

	// FindOrCreate a node for every loaded UClass, and set the UClass fields
	CreateNodesForLoadedClasses(ObjectClassRoot, ClassPathToNode);

	// Set the parent and child pointers
	for (TPair<FTopLevelAssetPath, TSharedPtr<FClassViewerNode>>& KVPair : ClassPathToNode)
	{
		TSharedPtr<FClassViewerNode>& Node = KVPair.Value;
		if (Node == ObjectClassRoot)
		{
			// No parent expected for the root class
			continue;
		}
		TSharedPtr<FClassViewerNode>* ParentNodePtr = nullptr;
		if (!Node->ParentClassPath.IsNull())
		{
			ParentNodePtr = ClassPathToNode.Find(Node->ParentClassPath);
			if(!ParentNodePtr)
			{
				FSoftObjectPath SoftParentClass(Node->ParentClassPath);
				UAssetRegistryHelpers::FixupRedirectedAssetPath(SoftParentClass);
				ParentNodePtr = ClassPathToNode.Find(FTopLevelAssetPath(SoftParentClass.ToString()));
			}
		}
		if (!ParentNodePtr)
		{
			UE_LOGF(LogEditorClassViewer, Warning, "Class %ls has parent %ls, but this parent is not found. The Class will not be shown in ClassViewer.",
				*KVPair.Key.ToString(), *Node->ParentClassPath.ToString());
			continue;
		}
		TSharedPtr<FClassViewerNode>& ParentNode = *ParentNodePtr;
		check(ParentNode);
		ParentNode->AddChild(Node);
	}

	// Recursively sort the children.
	SortChildren(ObjectClassRoot);

	// All viewers must refresh.
	ClassViewer::Helpers::RefreshAll();
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SClassViewer::Construct(const FArguments& InArgs, const FClassViewerInitializationOptions& InInitOptions )
{
	bNeedsRefresh = true;
	NumClasses = 0;
	
	// Listen for when view settings are changed
	UClassViewerSettings::OnSettingChanged().AddSP(this, &SClassViewer::HandleSettingChanged);

	InitOptions = InInitOptions;

	OnClassPicked = InArgs._OnClassPickedDelegate;

	bSaveExpansionStates = true;
	bPendingSetExpansionStates = false;

	ClassFilter = MakeShareable(new FClassViewerFilter(InitOptions));

	bEnableClassDynamicLoading = InInitOptions.bEnableClassDynamicLoading;

	EVisibility HeaderVisibility = (this->InitOptions.Mode == EClassViewerMode::ClassBrowsing)? EVisibility::Visible : EVisibility::Collapsed;

	// If set to default, decide what display mode to use.
	if( InitOptions.DisplayMode == EClassViewerDisplayMode::DefaultView )
	{
		// By default the Browser uses the tree view, the Picker the list. The option is available to users to force to another display mode when creating the Class Browser/Picker.
		if( InitOptions.Mode == EClassViewerMode::ClassBrowsing )
		{
			InitOptions.DisplayMode = EClassViewerDisplayMode::TreeView;
		}
		else
		{
			InitOptions.DisplayMode = EClassViewerDisplayMode::ListView;
		}
	}

	// Clear out the current set of custom filter options.
	CustomClassFilterOptions.Empty(InitOptions.ClassFilters.Num());

	// Gather additional filter options from any custom filters.
	TArray<TSharedRef<FClassViewerFilterOption>> FilterOptions;
	for (TSharedRef<IClassViewerFilter> CustomFilter : InitOptions.ClassFilters)
	{
		// Append this filter's options to the current set.
		CustomFilter->GetFilterOptions(FilterOptions);
		CustomClassFilterOptions.Append(FilterOptions);

		// Reset the temp array for the next pass.
		FilterOptions.Reset();
	}

	TSharedRef<SWidget> FiltersWidget = SNullWidget::NullWidget;
	// Build the top menu
	if(InitOptions.Mode == EClassViewerMode::ClassBrowsing)
	{
		FiltersWidget = 
		SNew(SComboButton)
		.ComboButtonStyle(FAppStyle::Get(), "GenericFilters.ComboButtonStyle")
		.ForegroundColor(FLinearColor::White)
		.ContentPadding(0)
		.ToolTipText(LOCTEXT("Filters_Tooltip", "Filter options for the Class Viewer."))
		.OnGetMenuContent(this, &SClassViewer::FillFilterEntries)
		.HasDownArrow(true)
		.ContentPadding(FMargin(1, 0))
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "GenericFilters.TextStyle")
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
				.Text(FText::FromString(FString(TEXT("\xf0b0"))) /*fa-filter*/)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2, 0, 0, 0)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "GenericFilters.TextStyle")
				.Text(LOCTEXT("Filters", "Filters"))
			]
		];
	}

	// Create the asset discovery indicator
	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");
	TSharedRef<SWidget> AssetDiscoveryIndicator = EditorWidgetsModule.CreateAssetDiscoveryIndicator(EAssetDiscoveryIndicatorScaleMode::Scale_Vertical);
	FOnContextMenuOpening OnContextMenuOpening;
	if ( InitOptions.Mode == EClassViewerMode::ClassBrowsing )
	{
		OnContextMenuOpening = FOnContextMenuOpening::CreateSP(this, &SClassViewer::BuildMenuWidget);
	}

	SAssignNew(ClassList, SListView<TSharedPtr< FClassViewerNode > >)
		.SelectionMode(ESelectionMode::Single)
		.ListItemsSource(&RootTreeItems)
		// Generates the actual widget for a tree item
		.OnGenerateRow(this, &SClassViewer::OnGenerateRowForClassViewer)
		// Generates the right click menu.
		.OnContextMenuOpening(OnContextMenuOpening)
		// Find out when the user selects something in the tree
		.OnSelectionChanged(this, &SClassViewer::OnClassViewerSelectionChanged)
		.HeaderRow
		(
			SNew(SHeaderRow)
			.Visibility(EVisibility::Collapsed)
			+ SHeaderRow::Column(TEXT("Class"))
			.DefaultLabel(NSLOCTEXT("ClassViewer", "Class", "Class"))
		);

	SAssignNew(ClassTree, STreeView<TSharedPtr< FClassViewerNode > >)
		.SelectionMode(ESelectionMode::Single)
		.TreeItemsSource(&RootTreeItems)
		// Called to child items for any given parent item
		.OnGetChildren(this, &SClassViewer::OnGetChildrenForClassViewerTree)
		// Called to handle recursively expanding/collapsing items
		.OnSetExpansionRecursive(this, &SClassViewer::SetAllExpansionStates_Helper)
		// Generates the actual widget for a tree item
		.OnGenerateRow(this, &SClassViewer::OnGenerateRowForClassViewer)
		// Generates the right click menu.
		.OnContextMenuOpening(OnContextMenuOpening)
		// Find out when the user selects something in the tree
		.OnSelectionChanged(this, &SClassViewer::OnClassViewerSelectionChanged)
		// Called when the expansion state of an item changes
		.OnExpansionChanged(this, &SClassViewer::OnClassViewerExpansionChanged)
		.HeaderRow
		(
			SNew(SHeaderRow)
			.Visibility(EVisibility::Collapsed)
			+ SHeaderRow::Column(TEXT("Class"))
			.DefaultLabel(NSLOCTEXT("ClassViewer", "Class", "Class"))
		);
	TSharedRef<STreeView<TSharedPtr< FClassViewerNode > > > ClassTreeView = ClassTree.ToSharedRef();
	TSharedRef<SListView<TSharedPtr< FClassViewerNode > > > ClassListView = ClassList.ToSharedRef();
	
	bool bHasTitle = InitOptions.ViewerTitleString.IsEmpty() == false;

	// Holds the bulk of the class viewer's sub-widgets, to be added to the widget after construction
	TSharedPtr< SWidget > ClassViewerContent;

	ClassViewerContent = 
	SNew(SBox)
	.MaxDesiredHeight(800.0f)
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(InitOptions.bShowBackgroundBorder ? "ToolPanel.GroupBorder" : "NoBorder"))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Visibility(bHasTitle ? EVisibility::Visible : EVisibility::Collapsed)
					.ColorAndOpacity(FAppStyle::GetColor("MultiboxHookColor"))
					.Text(InitOptions.ViewerTitleString)
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 2.0f)
				[
					FiltersWidget
				]
				+ SHorizontalBox::Slot()
				.Padding(2.0f, 2.0f, 6.0f, 2.0f)
				[
					SAssignNew(SearchBox, SSearchBox)
					.OnTextChanged( this, &SClassViewer::OnFilterTextChanged )
					.OnTextCommitted( this, &SClassViewer::OnFilterTextCommitted )
				]
				// View mode combo button
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 2.0f)
				[
					SAssignNew(ViewOptionsComboButton, SComboButton)
					.ContentPadding(0)
					.ForegroundColor(FSlateColor::UseForeground())
					.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
					.HasDownArrow(false)
					.OnGetMenuContent(this, &SClassViewer::GetViewButtonContent)
					.ButtonContent()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSeparator)
				.Visibility(HeaderVisibility)
			]

			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SOverlay)

				+SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.FillHeight(1.0f)
					[
						SNew(SScrollBorder, ClassTreeView)
						.Visibility(InitOptions.DisplayMode == EClassViewerDisplayMode::TreeView ? EVisibility::Visible : EVisibility::Collapsed)
						[
							ClassTreeView
						]
					]

					+SVerticalBox::Slot()
					.FillHeight(1.0f)
					[
						SNew(SScrollBorder, ClassListView)
						.Visibility(InitOptions.DisplayMode == EClassViewerDisplayMode::ListView ? EVisibility::Visible : EVisibility::Collapsed)
						[
							ClassListView
						]
					]
				]

				+SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Bottom)
				.Padding(FMargin(24, 0, 24, 0))
				[
					// Asset discovery indicator
					AssetDiscoveryIndicator
				]
			]

			// Bottom panel
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SNew(STextBlock)
				.Text(this, &SClassViewer::GetClassCountText)
			]
		]
	];

	if (ViewOptionsComboButton.IsValid())
	{
		ViewOptionsComboButton->SetVisibility(InitOptions.bAllowViewOptions ? EVisibility::Visible : EVisibility::Collapsed);
	}

	// When using a class picker in list-view mode, the widget will auto-focus the search box
	// and allow the up and down arrow keys to navigate and enter to pick without using the mouse ever
	if ( InitOptions.Mode == EClassViewerMode::ClassPicker && InitOptions.DisplayMode == EClassViewerDisplayMode::ListView )
	{
		this->ChildSlot
		[
			SNew(SListViewSelectorDropdownMenu<TSharedPtr<FClassViewerNode>>, SearchBox, ClassList)
			[
				ClassViewerContent.ToSharedRef()
			]
		];
	}
	else
	{
		this->ChildSlot
		[
			ClassViewerContent.ToSharedRef()
		];
	}

	// Construct the class hierarchy.
	ClassViewer::Helpers::ConstructClassHierarchy();

	// Only want filter options enabled in browsing mode.
	if( this->InitOptions.Mode == EClassViewerMode::ClassBrowsing )
	{
		// Default the "Only Placeable" checkbox to be checked, it will check "Only Actors"
		MenuPlaceableOnly_Execute();
	}

	ClassViewer::Helpers::PopulateClassviewerDelegate.AddSP(this, &SClassViewer::Refresh);

	// Request delayed setting of focus to the search box
	bPendingFocusNextFrame = true;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<SWidget> SClassViewer::GetContent()
{
	return SharedThis( this );
}

SClassViewer::~SClassViewer()
{
	ClassViewer::Helpers::PopulateClassviewerDelegate.RemoveAll(this);

	// Remove the listener for when view settings are changed
	UClassViewerSettings::OnSettingChanged().RemoveAll(this);
}

void SClassViewer::ClearSelection()
{
	ClassTree->ClearSelection();
}

void SClassViewer::OnGetChildrenForClassViewerTree( TSharedPtr<FClassViewerNode> InParent, TArray< TSharedPtr< FClassViewerNode > >& OutChildren )
{
	// Simply return the children, it's already setup.
	OutChildren = InParent->GetChildrenList();
}

void SClassViewer::OnClassViewerSelectionChanged( TSharedPtr<FClassViewerNode> Item, ESelectInfo::Type SelectInfo )
{
	// Do not act on selection change when it is for navigation
	if(SelectInfo == ESelectInfo::OnNavigation && InitOptions.DisplayMode == EClassViewerDisplayMode::ListView)
	{
		return;
	}

	// Sometimes the item is not valid anymore due to filtering.
	if(Item.IsValid() == false || Item->IsRestricted())
	{
		return;
	}

	if(InitOptions.Mode == EClassViewerMode::ClassBrowsing)
	{
		// Allows the user to right click in the level editor and select to place the selected class.
		GUnrealEd->SetCurrentClass( Item->Class.Get() );
	}
	else
	{
		UClass* Class = Item->Class.Get();

		// If the class is nullptr and UnloadedBlueprintData is valid then attempt to load it. UnloadedBlueprintData is invalid in the case of a "None" item.
		if ( bEnableClassDynamicLoading && !Class && Item->UnloadedBlueprintData.IsValid() )
		{
			ClassViewer::Helpers::LoadClass( Item );

			// Populate the tree/list so any changes to previously unloaded classes will be reflected.
			Refresh();
		}

		// Check if the item passes the filter
		if ( ( Item->Class.IsValid() || !Class ))
		{
			// Parent items might be displayed but filtered out by bPassesFilter, thus bPassesFilterRegardlessTextFilter makes sure to keep them selectable.
			// In addition, Item->bPassesFilter would be redundant here as bPassesFilterRegardlessTextFilter = true if bPassesFilter = true
			if (Item->bPassesFilterRegardlessTextFilter || Item->bPassesFilter)
			{
				OnClassPicked.ExecuteIfBound( Item->Class.Get() );
			}
			else
			{
				OnClassPicked.ExecuteIfBound( nullptr );
			}
		}
	}
}

void SClassViewer::OnClassViewerExpansionChanged(TSharedPtr<FClassViewerNode> Item, bool bExpanded)
{
	// Sometimes the item is not valid anymore due to filtering.
	if (Item.IsValid() == false || Item->IsRestricted())
	{
		return;
	}

	ExpansionStateMap.Add(*(Item->GetClassName()), bExpanded);
}

TSharedPtr< SWidget > SClassViewer::BuildMenuWidget()
{
	bool bIsBlueprint;
	bool bHasBlueprint;
	TArray< TSharedPtr< FClassViewerNode > > SelectedList;

	// Based upon which mode the viewer is in, pull the selected item.
	if( InitOptions.DisplayMode == EClassViewerDisplayMode::TreeView )
	{
		SelectedList = ClassTree->GetSelectedItems();
	}
	else
	{
		SelectedList = ClassList->GetSelectedItems();
	}

	// If there is no selected item, return a null widget.
	if(SelectedList.Num() == 0)
	{
		return SNullWidget::NullWidget;
	}

	// If it is NOT stale, it has not been set (meaning it was never valid but now is invalid).
	if( bEnableClassDynamicLoading && !SelectedList[0]->Class.IsStale() && !SelectedList[0]->Class.IsValid() && SelectedList[0]->UnloadedBlueprintData.IsValid() )
	{
		ClassViewer::Helpers::LoadClass(SelectedList[0]);

		// Populate the tree/list so any changes to previously unloaded classes will be reflected.
		Refresh();
	}

	// Get the class and it's info.
	RightClickClass = SelectedList[0]->Class.Get();
	RightClickBlueprint = SelectedList[0]->Blueprint.Get();
	ClassViewer::Helpers::GetClassInfo(RightClickClass, bIsBlueprint, bHasBlueprint);
	
	if(RightClickBlueprint)
	{
		bHasBlueprint = true;
	}

	return ClassViewer::Helpers::CreateMenu(RightClickClass, bIsBlueprint, bHasBlueprint);
}

TSharedRef< ITableRow > SClassViewer::OnGenerateRowForClassViewer( TSharedPtr<FClassViewerNode> Item, const TSharedRef< STableViewBase >& OwnerTable )
{	
	// If the item was accepted by the filter, leave it bright, otherwise dim it.
	float AlphaValue = Item->bPassesFilter? 1.0f : 0.5f;

	// If the item passed the filter, it may be from a match with hidden class name strings, update the search box text to retain highlighting of valid matching text
	FText SearchBoxTextForHighlight = SearchBox->GetText();
	TSharedPtr<FString> ClassNameDisplay = Item->GetClassName(InitOptions.NameTypeToDisplay);
	if (!SearchBoxTextForHighlight.IsEmpty() && (ClassNameDisplay->Find(*SearchBoxTextForHighlight.ToString()) == INDEX_NONE))
	{
		SearchBoxTextForHighlight = FText::FromString(UObjectBase::RemoveClassPrefix(*SearchBoxTextForHighlight.ToString()));
	}

	TSharedRef< SClassItem > ReturnRow = SNew(SClassItem, OwnerTable)
		.ClassName(ClassNameDisplay)
		.bIsPlaceable(Item->IsClassPlaceable())
		.HighlightText(SearchBoxTextForHighlight)
		.Font(Item->IsClassPlaceable()? FCoreStyle::Get().GetFontStyle("NormalFontItalic") : FCoreStyle::Get().GetFontStyle("NormalFont"))
		.AssociatedNode(Item)
		.bIsInClassViewer( InitOptions.Mode == EClassViewerMode::ClassBrowsing )
		.bDynamicClassLoading( bEnableClassDynamicLoading )
		.OnDragDetected(this, &SClassViewer::OnDragDetected)
		.OnClassItemDoubleClicked(FOnClassItemDoubleClickDelegate::CreateSP(this, &SClassViewer::ToggleExpansionState_Helper));

	// Expand the item if needed.
	if (!bPendingSetExpansionStates)
	{
		bool* bIsExpanded = ExpansionStateMap.Find(*(Item->GetClassName()));
		if (bIsExpanded && *bIsExpanded)
		{
			bPendingSetExpansionStates = true;
		}
	}

	return ReturnRow;
}

const TArray< TSharedPtr< FClassViewerNode > > SClassViewer::GetSelectedItems() const
{
	if ( InitOptions.DisplayMode == EClassViewerDisplayMode::ListView )
	{
		return ClassList->GetSelectedItems();
	}

	return ClassTree->GetSelectedItems();
}


const int SClassViewer::GetNumItems() const
{
	return NumClasses;
}

TSharedRef<SWidget> SClassViewer::GetViewButtonContent()
{
	// Get all menu extenders for this context menu from the content browser module

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr, nullptr, /*bCloseSelfOnly=*/ true);

	MenuBuilder.AddMenuEntry(LOCTEXT("ExpandAll", "Expand All"), LOCTEXT("ExpandAll_Tooltip", "Expands the entire tree"), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SClassViewer::SetAllExpansionStates, bool(true))), NAME_None, EUserInterfaceActionType::Button);
	MenuBuilder.AddMenuEntry(LOCTEXT("CollapseAll", "Collapse All"), LOCTEXT("CollapseAll_Tooltip", "Collapses the entire tree"), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SClassViewer::SetAllExpansionStates, bool(false))), NAME_None, EUserInterfaceActionType::Button);

	MenuBuilder.BeginSection("Filters", LOCTEXT("ClassViewerFiltersHeading", "Class Filters"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowInternalClassesOption", "Show Internal Classes"),
			LOCTEXT("ShowInternalClassesOptionToolTip", "Shows internal-use only classes in the view."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SClassViewer::ToggleShowInternalClasses),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SClassViewer::IsShowingInternalClasses)
				),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
			);

		for (const TSharedRef<FClassViewerFilterOption>& FilterOption : CustomClassFilterOptions)
		{
			MenuBuilder.AddMenuEntry(
				FilterOption->LabelText,
				FilterOption->ToolTipText,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SClassViewer::ToggleCustomFilterOption, FilterOption),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SClassViewer::IsCustomFilterOptionEnabled, FilterOption)
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("DeveloperViewType", LOCTEXT("DeveloperViewTypeHeading", "Developer Folder Filter"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("NoneDeveloperViewOption", "None"),
			LOCTEXT("NoneDeveloperViewOptionToolTip", "Filter classes to show no classes in developer folders."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SClassViewer::SetCurrentDeveloperViewType, EClassViewerDeveloperType::CVDT_None),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SClassViewer::IsCurrentDeveloperViewType, EClassViewerDeveloperType::CVDT_None)
				),
			NAME_None,
			EUserInterfaceActionType::RadioButton
			);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CurrentUserDeveloperViewOption", "Current Developer"),
			LOCTEXT("CurrentUserDeveloperViewOptionToolTip", "Filter classes to allow classes in the current user's development folder."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SClassViewer::SetCurrentDeveloperViewType, EClassViewerDeveloperType::CVDT_CurrentUser),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SClassViewer::IsCurrentDeveloperViewType, EClassViewerDeveloperType::CVDT_CurrentUser)
				),
			NAME_None,
			EUserInterfaceActionType::RadioButton
			);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AllUsersDeveloperViewOption", "All Developers"),
			LOCTEXT("AllUsersDeveloperViewOptionToolTip", "Filter classes to allow classes in all users' development folders."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SClassViewer::SetCurrentDeveloperViewType, EClassViewerDeveloperType::CVDT_All),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SClassViewer::IsCurrentDeveloperViewType, EClassViewerDeveloperType::CVDT_All)
				),
			NAME_None,
			EUserInterfaceActionType::RadioButton
			);

	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SClassViewer::SetCurrentDeveloperViewType(EClassViewerDeveloperType NewType)
{
	if (ensure((int)NewType < (int)EClassViewerDeveloperType::CVDT_Max) && NewType != GetDefault<UClassViewerSettings>()->DeveloperFolderType)
	{
		GetMutableDefault<UClassViewerSettings>()->DeveloperFolderType = NewType;
		GetMutableDefault<UClassViewerSettings>()->PostEditChange();
	}
}

EClassViewerDeveloperType SClassViewer::GetCurrentDeveloperViewType() const
{
	if (!InitOptions.bAllowViewOptions)
	{
		return EClassViewerDeveloperType::CVDT_All;
	}
	return GetDefault<UClassViewerSettings>()->DeveloperFolderType;
}

bool SClassViewer::IsCurrentDeveloperViewType(EClassViewerDeveloperType ViewType) const
{
	return GetCurrentDeveloperViewType() == ViewType;
}

void SClassViewer::GetInternalOnlyClasses(TArray<FSoftClassPath>& Classes)
{
	if (!InitOptions.bAllowViewOptions)
	{
		return;
	}
	Classes = GetDefault<UClassViewerProjectSettings>()->InternalOnlyClasses;
}

void SClassViewer::GetInternalOnlyPaths(TArray<FDirectoryPath>& Paths)
{
	if (!InitOptions.bAllowViewOptions)
	{
		return;
	}
	Paths = GetDefault<UClassViewerProjectSettings>()->InternalOnlyPaths;
}

FText SClassViewer::GetClassCountText() const
{

	const int32 NumAssets = GetNumItems();
	const int32 NumSelectedAssets = GetSelectedItems().Num();

	FText AssetCount = LOCTEXT("AssetCountLabelSingular", "1 item");

	if (NumSelectedAssets == 0)
	{
		if (NumAssets == 1)
		{
			AssetCount = LOCTEXT("AssetCountLabelSingular", "1 item");
		}
		else
		{
			AssetCount = FText::Format(LOCTEXT("AssetCountLabelPlural", "{0} items"), FText::AsNumber(NumAssets));
		}
	}
	else
	{
		if (NumAssets == 1)
		{
			AssetCount = FText::Format(LOCTEXT("AssetCountLabelSingularPlusSelection", "1 item ({0} selected)"), FText::AsNumber(NumSelectedAssets));
		}
		else
		{
			AssetCount = FText::Format(LOCTEXT("AssetCountLabelPluralPlusSelection", "{0} items ({1} selected)"), FText::AsNumber(NumAssets), FText::AsNumber(NumSelectedAssets));
		}
	}

	return AssetCount;
}


void SClassViewer::ExpandRootNodes()
{
	for (int32 NodeIdx = 0; NodeIdx < RootTreeItems.Num(); ++NodeIdx)
	{
		ExpansionStateMap.Add(*(RootTreeItems[NodeIdx]->GetClassName()), true);
		ClassTree->SetItemExpansion(RootTreeItems[NodeIdx], true);
	}
}

FReply SClassViewer::OnDragDetected( const FGeometry& Geometry, const FPointerEvent& PointerEvent )
{
	if(InitOptions.Mode == EClassViewerMode::ClassBrowsing)
	{
		const TArray< TSharedPtr< FClassViewerNode > > SelectedItems = GetSelectedItems();

		if ( SelectedItems.Num() > 0 && SelectedItems[0].IsValid() )
		{
			TSharedRef< FClassViewerNode > Item = SelectedItems[0].ToSharedRef();

			// If there is no class then we must spawn an FAssetDragDropOp so the class will be loaded when dropped.
			if ( UClass* Class = Item->Class.Get() )
			{
				// Spawn a loaded blueprint just like any other asset from the Content Browser.
				if ( Item->Blueprint.IsValid() )
				{
					const FAssetData AssetData(Item->Blueprint.Get());
					return FReply::Handled().BeginDragDrop(FContentBrowserDataDragDropOp::Legacy_New(MakeArrayView(&AssetData, 1)));
				}
				else
				{
					// Add the UClass associated with this item to the drag event being spawned.
					return FReply::Handled().BeginDragDrop(FClassDragDropOp::New(MakeWeakObjectPtr(Class)));
				}	
			}
			else if (!Item->BlueprintAssetPath.IsNull())
			{
				IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();

				// Pull asset data out of asset registry
				const FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(Item->BlueprintAssetPath);
				return FReply::Handled().BeginDragDrop(FContentBrowserDataDragDropOp::Legacy_New(MakeArrayView(&AssetData, 1)));
			}
		}
	}

	return FReply::Unhandled();
}

void SClassViewer::OnOpenBlueprintTool()
{
	ClassViewer::Helpers::OpenBlueprintTool(RightClickBlueprint);
}

void SClassViewer::FindInContentBrowser()
{
	ClassViewer::Helpers::FindInContentBrowser(RightClickBlueprint, RightClickClass);
}

void SClassViewer::OnFilterTextChanged( const FText& InFilterText )
{
	// Update the compiled filter and report any syntax error information back to the user
	ClassFilter->TextFilter->SetFilterText(InFilterText);
	SearchBox->SetError(ClassFilter->TextFilter->GetFilterErrorText());

	// Repopulate the list to show only what has not been filtered out.
	Refresh();
}

void SClassViewer::OnFilterTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		if (InitOptions.Mode == EClassViewerMode::ClassPicker)
		{
			TArray< TSharedPtr< FClassViewerNode > > SelectedList = ClassList->GetSelectedItems();
			TSharedPtr< FClassViewerNode > FirstSelected;

			UClass* Class = nullptr;
			
			if (SelectedList.Num() > 0)
			{
				FirstSelected = SelectedList[0];
				Class = FirstSelected->Class.Get();

				// If the class is nullptr and UnloadedBlueprintData is valid then attempt to load it. UnloadedBlueprintData is invalid in the case of a "None" item.
				if ( bEnableClassDynamicLoading && Class == nullptr && FirstSelected->UnloadedBlueprintData.IsValid())
				{
					ClassViewer::Helpers::LoadClass(FirstSelected);
					Class = FirstSelected->Class.Get();
				}

				// Check if the item passes the filter, parent items might be displayed but filtered out and thus not desired to be selected.
				if (Class && FirstSelected->bPassesFilter == true)
				{
					OnClassPicked.ExecuteIfBound(Class);
				}
			}
		}
	}
}

bool SClassViewer::Menu_CanExecute() const
{
	return true;
}

void SClassViewer::MenuActorsOnly_Execute()
{
	InitOptions.bIsActorsOnly = !InitOptions.bIsActorsOnly;

	// "Placeable Only" cannot be true when "Actors Only" is false.
	if (!InitOptions.bIsActorsOnly)
	{
		InitOptions.bIsPlaceableOnly = false;
	}

	Refresh();
}

bool SClassViewer::MenuActorsOnly_IsChecked() const
{
	return InitOptions.bIsActorsOnly;
}

void SClassViewer::MenuPlaceableOnly_Execute()
{
	InitOptions.bIsPlaceableOnly = !InitOptions.bIsPlaceableOnly;

	// "Actors Only" must be true when "Placeable Only" is true.
	if (InitOptions.bIsPlaceableOnly)
	{
		InitOptions.bIsActorsOnly = true;
	}

	Refresh();
}

bool SClassViewer::MenuPlaceableOnly_IsChecked() const
{
	return InitOptions.bIsPlaceableOnly;
}

void SClassViewer::MenuBlueprintBasesOnly_Execute()
{
	InitOptions.bIsBlueprintBaseOnly = !InitOptions.bIsBlueprintBaseOnly;

	Refresh();
}

bool SClassViewer::MenuBlueprintBasesOnly_IsChecked() const
{
	return InitOptions.bIsBlueprintBaseOnly;
}

TSharedRef<SWidget> SClassViewer::FillFilterEntries()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	MenuBuilder.BeginSection("ClassViewerFilterEntries");
	{
		MenuBuilder.AddMenuEntry( LOCTEXT("ActorsOnly", "Actors Only"), LOCTEXT( "ActorsOnly_Tooltip", "Filter the Class Viewer to show only actors" ), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(this, &SClassViewer::MenuActorsOnly_Execute), FCanExecuteAction::CreateRaw(this, &SClassViewer::Menu_CanExecute), FIsActionChecked::CreateRaw(this, &SClassViewer::MenuActorsOnly_IsChecked)), NAME_None, EUserInterfaceActionType::Check );
		MenuBuilder.AddMenuEntry( LOCTEXT("PlaceableOnly", "Placeable Only"), LOCTEXT( "PlaceableOnly_Tooltip", "Filter the Class Viewer to show only placeable actors." ), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(this, &SClassViewer::MenuPlaceableOnly_Execute), FCanExecuteAction::CreateRaw(this, &SClassViewer::Menu_CanExecute), FIsActionChecked::CreateRaw(this, &SClassViewer::MenuPlaceableOnly_IsChecked)), NAME_None, EUserInterfaceActionType::Check );
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("ClassViewerFilterEntries2");
	{
		MenuBuilder.AddMenuEntry( LOCTEXT("BlueprintsOnly", "Blueprint Class Bases Only"), LOCTEXT( "BlueprinsOnly_Tooltip", "Filter the Class Viewer to show only base blueprint classes." ), FSlateIcon(), FUIAction(FExecuteAction::CreateRaw(this, &SClassViewer::MenuBlueprintBasesOnly_Execute), FCanExecuteAction::CreateRaw(this, &SClassViewer::Menu_CanExecute), FIsActionChecked::CreateRaw(this, &SClassViewer::MenuBlueprintBasesOnly_IsChecked)), NAME_None, EUserInterfaceActionType::Check );
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SClassViewer::SetAllExpansionStates(bool bInExpansionState)
{
	// Go through all the items in the root of the tree and recursively visit their children to set every item in the tree.
	for(int32 ChildIndex = 0; ChildIndex < RootTreeItems.Num(); ChildIndex++)
	{
		SetAllExpansionStates_Helper( RootTreeItems[ChildIndex], bInExpansionState );
	}
}

void SClassViewer::SetAllExpansionStates_Helper(TSharedPtr< FClassViewerNode > InNode, bool bInExpansionState)
{
	ClassTree->SetItemExpansion(InNode, bInExpansionState);

	// Recursively go through the children.
	for(int32 ChildIndex = 0; ChildIndex < InNode->GetChildrenList().Num(); ChildIndex++)
	{
		SetAllExpansionStates_Helper( InNode->GetChildrenList()[ChildIndex], bInExpansionState );
	}
}

void SClassViewer::ToggleExpansionState_Helper(TSharedPtr< FClassViewerNode > InNode)
{
	bool bExpanded = ClassTree->IsItemExpanded( InNode );
	ClassTree->SetItemExpansion(InNode, !bExpanded);
}

bool SClassViewer::ExpandFilteredInNodes(TSharedPtr<FClassViewerNode> InNode)
{
	bool bShouldExpand(InNode->bPassesFilter);

	for(int32 ChildIdx = 0; ChildIdx < InNode->GetChildrenList().Num(); ChildIdx++)
	{
		bShouldExpand |= ExpandFilteredInNodes(InNode->GetChildrenList()[ChildIdx]);
	}

	if(bShouldExpand)
	{
		ClassTree->SetItemExpansion(InNode, true);
	}

	return bShouldExpand;
}

void SClassViewer::MapExpansionStatesInTree( TSharedPtr<FClassViewerNode> InItem )
{
	ExpansionStateMap.Add( *(InItem->GetClassName()), ClassTree->IsItemExpanded( InItem ) );

	// Map out all the children, this will be done recursively.
	for( int32 ChildIdx(0); ChildIdx < InItem->GetChildrenList().Num(); ++ChildIdx )
	{
		MapExpansionStatesInTree( InItem->GetChildrenList()[ChildIdx] );
	}
}

void SClassViewer::SetExpansionStatesInTree( TSharedPtr<FClassViewerNode> InItem )
{
	bool* bIsExpanded = ExpansionStateMap.Find( *(InItem->GetClassName()) );
	if( bIsExpanded )
	{
		ClassTree->SetItemExpansion( InItem, *bIsExpanded );

		// No reason to set expansion states if the parent is not expanded, it does not seem to do anything.
		if( *bIsExpanded )
		{
			for( int32 ChildIdx(0); ChildIdx < InItem->GetChildrenList().Num(); ++ChildIdx )
			{
				SetExpansionStatesInTree( InItem->GetChildrenList()[ChildIdx] );
			}
		}
	}
	else
	{
		// Default to no expansion.
		ClassTree->SetItemExpansion( InItem, false );
	}
}

int32 SClassViewer::CountTreeItems(FClassViewerNode* Node)
{
	if (Node == nullptr)
	{
		return 0;
	}
	int32 Count = 1;
	TArray<TSharedPtr<FClassViewerNode>>& ChildArray = Node->GetChildrenList();
	for (int32 i = 0; i < ChildArray.Num(); i++)
	{
		Count += CountTreeItems(ChildArray[i].Get());
	}
	return Count;
}

void SClassViewer::Populate()
{
	TArray<FTopLevelAssetPath> PreviousSelection;
	{
		TArray<TSharedPtr<FClassViewerNode>> SelectedItems = GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			for (TSharedPtr<FClassViewerNode>& Node : SelectedItems)
			{
				if (Node.IsValid())
				{
					PreviousSelection.Add(Node->ClassPath);
				}
			}
		}
	}

	bPendingSetExpansionStates = false;

	// If showing a class tree, we may need to save expansion states.
	if(InitOptions.DisplayMode == EClassViewerDisplayMode::TreeView)
	{
		if( bSaveExpansionStates )
		{
			for( int32 ChildIdx(0); ChildIdx < RootTreeItems.Num(); ++ChildIdx )
			{
				// Check if the item is actually expanded or if it's only expanded because it is root level.
				bool* bIsExpanded = ExpansionStateMap.Find( *(RootTreeItems[ChildIdx]->GetClassName()) );
				if((bIsExpanded && !*bIsExpanded) || !bIsExpanded)
				{
					ClassTree->SetItemExpansion( RootTreeItems[ChildIdx], false );
				}

				// Recursively map out the expansion state of the tree-node.
				MapExpansionStatesInTree( RootTreeItems[ChildIdx] );
			}
		}

		// This is set to false before the call to populate when it is not desired.
		bSaveExpansionStates = true;
	}

	// Empty the tree out so it can be redone.
	RootTreeItems.Empty();

	TArray<FSoftClassPath> InternalClassNames;
	// If we aren't showing the internal classes, then we need to know what classes to consider Internal Only, so let's gather them up from the settings object.
	if (!IsShowingInternalClasses())
	{
		GetInternalOnlyPaths(ClassFilter->InternalPaths);
		GetInternalOnlyClasses(InternalClassNames);

		// Take the package names for the internal only classes and convert them into their UClass
		for (int i = 0; i < InternalClassNames.Num(); i++)
		{
			FTopLevelAssetPath PackageClassName(InternalClassNames[i].ToString());
			const TSharedPtr<FClassViewerNode> ClassNode = ClassViewer::Helpers::ClassHierarchy->FindNodeByGeneratedClassPath(ClassViewer::Helpers::ClassHierarchy->GetObjectRootNode(), PackageClassName);

			if (ClassNode.IsValid())
			{
				ClassFilter->InternalClasses.Add(ClassNode->Class.Get());
			}
		}
	}

	

	// Based on if the list or tree is visible we create what will be displayed differently.
	if(InitOptions.DisplayMode == EClassViewerDisplayMode::TreeView)
	{
		// The root node for the tree, will be "Object" which we will skip.
		TSharedPtr<FClassViewerNode> RootNode;

		// Get the class tree, passing in certain filter options.
		ClassViewer::Helpers::GetClassTree(RootNode, ClassFilter, InitOptions);

		// Check if we will restore expansion states, we will not if there is filtering happening.
		const bool bRestoreExpansionState = ClassFilter->TextFilter->GetFilterType() == ETextFilterExpressionType::Empty;

		if(InitOptions.bShowObjectRootClass)
		{
			RootTreeItems.Add(RootNode);

			if( bRestoreExpansionState )
			{
				SetExpansionStatesInTree( RootNode );
			}

			// Expand any items that pass the filter.
			if(ClassFilter->TextFilter->GetFilterType() != ETextFilterExpressionType::Empty)
			{
				ExpandFilteredInNodes(RootNode);
			}
		}
		else
		{
			// Add all the children of the "Object" root.
			for(int32 ChildIndex = 0; ChildIndex < RootNode->GetChildrenList().Num(); ChildIndex++)
			{
				RootTreeItems.Add(RootNode->GetChildrenList()[ChildIndex]);
				if( bRestoreExpansionState )
				{
					SetExpansionStatesInTree( RootTreeItems[ChildIndex] );
				}

				// Expand any items that pass the filter.
				if(ClassFilter->TextFilter->GetFilterType() != ETextFilterExpressionType::Empty)
				{
					ExpandFilteredInNodes(RootNode->GetChildrenList()[ChildIndex]);
				}
			}
		}

		// Only display this option if the user wants it and in Picker Mode.
		if(InitOptions.bShowNoneOption && InitOptions.Mode == EClassViewerMode::ClassPicker)
		{
			// @todo - It would seem smart to add this in before the other items, since it needs to be on top. However, that causes strange issues with saving/restoring expansion states. 
			// This is likely not very efficient since the list can have hundreds and even thousands of items.
			RootTreeItems.Insert(CreateNoneOption(), 0);			
		}

		NumClasses = 0;
		for (int32 i = 0; i < RootTreeItems.Num(); i++)
		{
			NumClasses += CountTreeItems(RootTreeItems[i].Get());
		}

		// Now that new items are in the tree, we need to request a refresh.
		ClassTree->RequestTreeRefresh();

		TSharedPtr<FClassViewerNode> ClassNode;
		TSharedPtr<FClassViewerNode> ExpandNode;
		if (PreviousSelection.Num() > 0)
		{
			ClassNode = ClassViewer::Helpers::ClassHierarchy->FindNodeByGeneratedClassPath(RootNode, PreviousSelection[0]);
			ExpandNode = ClassNode ? ClassNode->GetParentNode() : nullptr;
		}
		else if (InitOptions.InitiallySelectedClass)
		{
			UClass* CurrentClass = InitOptions.InitiallySelectedClass;
			InitOptions.InitiallySelectedClass = nullptr;

			TArray<UClass*> ClassHierarchy;
			while (CurrentClass)
			{
				ClassHierarchy.Add(CurrentClass);
				CurrentClass = CurrentClass->GetSuperClass();
			}

			ClassNode = RootNode;
			for (int32 Index = ClassHierarchy.Num() - 2; Index >= 0; --Index)
			{
				for (const TSharedPtr<FClassViewerNode>& ChildClassNode : ClassNode->GetChildrenList())
				{
					UClass* ChildClass = ChildClassNode->Class.Get();
					if (ChildClass == ClassHierarchy[Index])
					{
						ClassNode = ChildClassNode;
						break;
					}
				}
			}
			ExpandNode = ClassNode;
		}

		for (; ExpandNode; ExpandNode = ExpandNode->GetParentNode())
		{
			ClassTree->SetItemExpansion(ExpandNode, true);
		}

		if (ClassNode)
		{
			ClassTree->SetSelection(ClassNode);
			ClassTree->RequestScrollIntoView(ClassNode);
		}
	}
	else
	{
		// Get the class list, passing in certain filter options.
		ClassViewer::Helpers::GetClassList(RootTreeItems, ClassFilter, InitOptions);

		if (InitOptions.ClassViewerSortPredicate)
		{
			RootTreeItems.Sort([this](const TSharedPtr<FClassViewerNode>& A, const TSharedPtr<FClassViewerNode>& B)
			{
				FClassViewerSortElementInfo InfoA (A->Class, A->GetClassName(false), A->GetClassName(true));
				FClassViewerSortElementInfo InfoB (B->Class, B->GetClassName(false), B->GetClassName(true));
				return InitOptions.ClassViewerSortPredicate(InfoA, InfoB);
			});
		}
		else
		{
			// Sort the list alphabetically.
			RootTreeItems.Sort(FClassViewerNodeNameLess(InitOptions.NameTypeToDisplay));
		}

		// Only display this option if the user wants it and in Picker Mode.
		if(InitOptions.bShowNoneOption && InitOptions.Mode == EClassViewerMode::ClassPicker)
		{
			// @todo - It would seem smart to add this in before the other items, since it needs to be on top. However, that causes strange issues with saving/restoring expansion states. 
			// This is likely not very efficient since the list can have hundreds and even thousands of items.
			RootTreeItems.Insert(CreateNoneOption(), 0);
		}

		NumClasses = 0;
		for (int32 i = 0; i < RootTreeItems.Num(); i++)
		{
			NumClasses += CountTreeItems(RootTreeItems[i].Get());
		}

		// Now that new items are in the list, we need to request a refresh.
		ClassList->RequestListRefresh();

		FString ClassPathNameToSelect;
		if (PreviousSelection.Num() > 0)
		{
			ClassPathNameToSelect = PreviousSelection[0].ToString();
		}
		else if (InitOptions.InitiallySelectedClass)
		{
			ClassPathNameToSelect = InitOptions.InitiallySelectedClass->GetPathName();
		}

		if (ClassPathNameToSelect.Len() > 0)
		{
			if(TSharedPtr<FClassViewerNode>* ClassNode = RootTreeItems.FindByPredicate([ClassPathNameToSelect](const TSharedPtr< FClassViewerNode > InClassNode) { return InClassNode->Class.IsValid() && (InClassNode->Class->GetPathName() == ClassPathNameToSelect); }))
			{
				ClassList->SetSelection(*ClassNode);
				ClassList->RequestScrollIntoView(*ClassNode);
			}
			InitOptions.InitiallySelectedClass = nullptr;
		}
	}
}

FReply SClassViewer::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	// Forward key down to class tree
	return ClassTree->OnKeyDown(MyGeometry,InKeyEvent);
}


FReply SClassViewer::OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent )
{
	if (InFocusEvent.GetCause() == EFocusCause::Navigation)
	{
		FSlateApplication::Get().SetKeyboardFocus(SearchBox.ToSharedRef(), EFocusCause::SetDirectly);
	}
	
	return FReply::Unhandled();
}

bool SClassViewer::SupportsKeyboardFocus() const 
{
	return true;
}

void SClassViewer::RequestPopulateClassHierarchy()
{
	ClassViewer::Helpers::RequestPopulateClassHierarchy();
}

void SClassViewer::DestroyClassHierarchy()
{
	ClassViewer::Helpers::DestroyClassHierachy();
}

TSharedPtr<FClassViewerNode> SClassViewer::CreateNoneOption()
{
	TSharedPtr<FClassViewerNode> NoneItem = MakeShared<FClassViewerNode>("None", "None");

	// The item "passes" the filter so it does not appear grayed out.
	NoneItem->bPassesFilter = true;
	NoneItem->bPassesFilterRegardlessTextFilter = true;

	return NoneItem;
}

void SClassViewer::Refresh()
{
	bNeedsRefresh = true;
}

void SClassViewer::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{	
	// Will populate the class hierarchy as needed.
	ClassViewer::Helpers::PopulateClassHierarchy();

	// Move focus to search box
	if (bPendingFocusNextFrame && SearchBox.IsValid())
	{
		FWidgetPath WidgetToFocusPath;
		FSlateApplication::Get().GeneratePathToWidgetUnchecked( SearchBox.ToSharedRef(), WidgetToFocusPath );
		FSlateApplication::Get().SetKeyboardFocus( WidgetToFocusPath, EFocusCause::SetDirectly );
		bPendingFocusNextFrame = false;
	}

	if (bNeedsRefresh)
	{
		bNeedsRefresh = false;
		Populate();

		if (InitOptions.bExpandRootNodes)
		{
			ExpandRootNodes();
		}

		if (InitOptions.bExpandAllNodes)
		{
			SetAllExpansionStates(true);
		}

		// Scroll the first item into view if applicable
		const TArray<TSharedPtr<FClassViewerNode>> SelectedItems = GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			ClassTree->RequestScrollIntoView(SelectedItems[0]);
		}
	}

	if (bPendingSetExpansionStates)
	{
		check(RootTreeItems.Num() > 0);
		SetExpansionStatesInTree(RootTreeItems[0]);
		bPendingSetExpansionStates = false;
	}
}

bool SClassViewer::IsClassAllowed(const UClass* InClass) const
{
	return ClassFilter->IsClassAllowed(InitOptions, InClass, ClassFilter->FilterFunctions);
}

void SClassViewer::HandleSettingChanged(FName PropertyName)
{
	if ((PropertyName == "DisplayInternalClasses") ||
		(PropertyName == "DeveloperFolderType") ||
		(PropertyName == NAME_None))	// @todo: Needed if PostEditChange was called manually, for now
	{
		Refresh();
	}
}

void SClassViewer::ToggleShowInternalClasses()
{
	check(IsToggleShowInternalClassesAllowed());
	GetMutableDefault<UClassViewerSettings>()->DisplayInternalClasses = !GetDefault<UClassViewerSettings>()->DisplayInternalClasses;
	GetMutableDefault<UClassViewerSettings>()->PostEditChange();
}

bool SClassViewer::IsToggleShowInternalClassesAllowed() const
{
	return InitOptions.bAllowViewOptions;
}

bool SClassViewer::IsShowingInternalClasses() const
{
	if (!InitOptions.bAllowViewOptions)
	{
		return true;
	}
	return IsToggleShowInternalClassesAllowed() ? GetDefault<UClassViewerSettings>()->DisplayInternalClasses : false;
}

void SClassViewer::ToggleCustomFilterOption(TSharedRef<FClassViewerFilterOption> FilterOption)
{
	FilterOption->bEnabled = !FilterOption->bEnabled;

	if (FilterOption->OnOptionChanged.IsBound())
	{
		FilterOption->OnOptionChanged.Execute(FilterOption->bEnabled);
	}

	Refresh();
}

bool SClassViewer::IsCustomFilterOptionEnabled(TSharedRef<FClassViewerFilterOption> FilterOption) const
{
	return FilterOption->bEnabled;
}

#undef LOCTEXT_NAMESPACE
