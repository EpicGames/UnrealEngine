// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnloadedBlueprintData.h"

#include "ClassViewerNode.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "HAL/PlatformCrt.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtrTemplates.h"


FUnloadedBlueprintData::FUnloadedBlueprintData(TWeakPtr<FClassViewerNode> InClassViewerNode)
	: ClassViewerNode(InClassViewerNode)
{
}

bool FUnloadedBlueprintData::HasAnyClassFlags( uint32 InFlagsToCheck ) const
{
	return (ClassFlags & InFlagsToCheck) != 0;
}

bool FUnloadedBlueprintData::HasAllClassFlags( uint32 InFlagsToCheck ) const
{
	return ((ClassFlags & InFlagsToCheck) == InFlagsToCheck);
}

void FUnloadedBlueprintData::SetClassFlags(uint32 InFlags)
{
	ClassFlags = InFlags;
}

bool FUnloadedBlueprintData::IsChildOf(const UClass* InClass) const
{
	TSharedPtr<FClassViewerNode> CurrentNode = ClassViewerNode.Pin()->GetParentNode();

	// Keep going through parents till you find an invalid.
	while (CurrentNode.IsValid())
	{
		if (CurrentNode->Class.Get() == InClass)
		{
			return true;
		}
		CurrentNode = CurrentNode->GetParentNode();
	}

	return false;
}

bool FUnloadedBlueprintData::ImplementsInterface(const UClass* InInterface) const
{
	FString InterfacePath = InInterface->GetPathName();
	// Does this blueprint implement the interface directly?
	for (const FString& DirectlyImplementedInterface : ImplementedInterfaces)
	{
		if (DirectlyImplementedInterface == InterfacePath)
		{
			return true;
		}
	}

	// If not, does a parent class implement the interface?
	TSharedPtr<FClassViewerNode> CurrentNode = ClassViewerNode.Pin()->GetParentNode();
	while (CurrentNode.IsValid())
	{
		if (CurrentNode->Class.IsValid() && CurrentNode->Class->ImplementsInterface(InInterface))
		{
			return true;
		}
		else if (CurrentNode->UnloadedBlueprintData.IsValid() && CurrentNode->UnloadedBlueprintData->ImplementsInterface(InInterface))
		{
			return true;
		}
		CurrentNode = CurrentNode->GetParentNode();
	}

	return false;
}

bool FUnloadedBlueprintData::IsA(const UClass* InClass) const
{
	// Unloaded blueprints will always return true for IsA(UBlueprintGeneratedClass::StaticClass). With that in mind, even though we do not have the exact class, we can use that knowledge as a basis for a check.
	return ((UObject*)UBlueprintGeneratedClass::StaticClass())->IsA(InClass);
}

const UClass* FUnloadedBlueprintData::GetClassWithin() const
{
	TSharedPtr<FClassViewerNode> CurrentNode = ClassViewerNode.Pin()->GetParentNode();

	while (CurrentNode.IsValid())
	{
		// The class field will be invalid for unloaded classes.
		// However, it should be valid once we've hit a loaded class or a Native class.
		// Assuming BP cannot change ClassWithin data, this should be safe.
		if (CurrentNode->Class.IsValid())
		{
			return CurrentNode->Class->ClassWithin;
		}

		CurrentNode = CurrentNode->GetParentNode();
	}

	return nullptr;
}

const UClass* FUnloadedBlueprintData::GetNativeParent() const
{
	TSharedPtr<FClassViewerNode> CurrentNode = ClassViewerNode.Pin()->GetParentNode();

	while (CurrentNode.IsValid())
	{
		// The class field will be invalid for unloaded classes.
		// However, it should be valid once we've hit a loaded class or a Native class.
		// Assuming BP cannot change ClassWithin data, this should be safe.
		if (CurrentNode->Class.IsValid() && CurrentNode->Class->HasAnyClassFlags(CLASS_Native))
		{
			return CurrentNode->Class.Get();
		}

		CurrentNode = CurrentNode->GetParentNode();
	}

	return nullptr;
}

TSharedPtr<FString> FUnloadedBlueprintData::GetClassName() const
{
	if (ClassViewerNode.IsValid())
	{
		return ClassViewerNode.Pin()->GetClassName();
	}

	return TSharedPtr<FString>();
}

FTopLevelAssetPath FUnloadedBlueprintData::GetClassPathName() const
{
	if (ClassViewerNode.IsValid())
	{
		return ClassViewerNode.Pin()->ClassPath;
	}

	return FTopLevelAssetPath();
}

const TWeakPtr<FClassViewerNode>& FUnloadedBlueprintData::GetClassViewerNode() const
{
	return ClassViewerNode;
}

void FUnloadedBlueprintData::AddImplementedInterface(const FString& InterfaceName)
{
	ImplementedInterfaces.Add(InterfaceName);
}
