// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClassViewerFilter.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/TopLevelAssetPath.h"

class FClassViewerNode;
class UClass;

class FUnloadedBlueprintData : public IUnloadedBlueprintData
{
public:
	FUnloadedBlueprintData(TWeakPtr<FClassViewerNode> InClassViewerNode);

	virtual ~FUnloadedBlueprintData() = default;

	virtual bool HasAnyClassFlags(uint32 InFlagsToCheck) const override;

	virtual bool HasAllClassFlags(uint32 InFlagsToCheck) const override;

	virtual void SetClassFlags(uint32 InFlags) override;

	virtual bool ImplementsInterface(const UClass* InInterface) const override;

	virtual bool IsChildOf(const UClass* InClass) const override;

	virtual bool IsA(const UClass* InClass) const override;

	virtual void SetNormalBlueprintType(bool bInNormalBPType) override { bNormalBlueprintType = bInNormalBPType; }

	virtual bool IsNormalBlueprintType() const override { return bNormalBlueprintType; }

	virtual TSharedPtr<FString> GetClassName() const override;

	virtual FTopLevelAssetPath GetClassPathName() const override;

	virtual const UClass* GetClassWithin() const override;

	virtual const UClass* GetNativeParent() const override;

	/** Retrieves the Class Viewer node this data is associated with. */
	const TWeakPtr<FClassViewerNode>& GetClassViewerNode() const;

	/** Adds the path of an interface that this blueprint implements directly. */
	void AddImplementedInterface(const FString& InterfacePath);

private:
	/** Flags for the class. */
	uint32 ClassFlags = CLASS_None;

	/** Is this a normal blueprint type? */
	bool bNormalBlueprintType;

	/** The full paths for all directly implemented interfaces for this class. */
	TArray<FString> ImplementedInterfaces;

	/** The node this class is contained in, used to gather hierarchical data as needed. */
	TWeakPtr<FClassViewerNode> ClassViewerNode;
};
