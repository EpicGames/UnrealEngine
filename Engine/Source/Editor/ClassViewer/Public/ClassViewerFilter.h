// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Engine/EngineTypes.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Settings/ClassViewerSettings.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/TopLevelAssetPath.h"

#define UE_API CLASSVIEWER_API

class FClassViewerFilterFuncs;
class FClassViewerInitializationOptions;
class FClassViewerNode;
class FString;
class FTextFilterExpressionEvaluator;
class IAssetReferenceFilter;
class IAssetRegistry;
class UClass;
class UObject;

/** Delegate used to respond to a filter option change. The argument will indicate whether the option was enabled or disabled. */
DECLARE_DELEGATE_OneParam(FOnClassViewerFilterOptionChanged, bool);

/** Called to check if a class should be filtered out by external code. Return true to exclude the class from the view. */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnShouldFilterClass, const UClass* /*Class*/);

/** Used to define a custom class viewer filter option. */
class FClassViewerFilterOption
{
public:
	/** Whether or not the option is currently enabled (default = true). */
	bool bEnabled = true;

	/** Localized label text to use for the option in the class viewer's filter menu. */
	TAttribute<FText> LabelText;

	/** Localized tooltip text to use for the option in the class viewer's filter menu. */
	TAttribute<FText> ToolTipText;

	/** Optional external delegate that will be invoked when this filter option is changed. */
	FOnClassViewerFilterOptionChanged OnOptionChanged;
};

/** Interface class for creating filters for the Class Viewer. */
class IClassViewerFilter
{
public:
	virtual ~IClassViewerFilter() = default;

	/**
	 * Checks if a class is allowed by this filter.
	 *
	 * @param InInitOptions				The Class Viewer/Picker options.
	 * @param InClass					The class to be tested.
	 * @param InFilterFuncs				Useful functions for filtering.
	 */
	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs ) = 0;

	/**
	 * Checks if a class is allowed by this filter.
	 *
	 * @param InInitOptions				The Class Viewer/Picker options.
	 * @param InUnloadedClassData		The information for the unloaded class to be tested.
	 * @param InFilterFuncs				Useful functions for filtering.
	 */
	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const class IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs) = 0;

	/**
	 * Can be optionally implemented to gather additional filter flags. If present, these
	 * will be added into the View Options menu where they can be toggled on/off by the user.
	 * 
	 * Example:
	 *	In the following case, we define a custom filter option to allow the user to toggle the entire filter on/off. This is
	 *	synced with a global config setting (MyFilterConfigSettings->bIsCustomFilterEnabled) that is checked at filtering time.
	 * 
	 *		void FMyCustomClassFilter::GetFilterOptions(TArray<TSharedRef<FClassViewerFilterOption>>& OutFilterOptions)
	 *		{
	 *			TSharedRef<FClassViewerFilterOption> MyFilterOption = MakeShared<FClassViewerFilterOption>();
	 *			MyFilterOption->bEnabled = MyFilterConfigSettings->bIsCustomFilterEnabled;
	 *			MyFilterOption->LabelText = LOCTEXT("MyCustomFilterOptionLabel", "My Custom Filter");
	 *			MyFilterOption->ToolTipText = LOCTEXT("MyCustomFilterOptionToolTip", "Enable or disable my custom class filter.");
	 *			MyFilterOption->OnOptionChanged = FOnClassViewerFilterOptionChanged::CreateSP(this, &FMyCustomClassFilter::OnOptionChanged);
	 * 
	 *			OutFilterOptions.Add(MyFilterOption);
	 *		}
	 *		
	 *		void FMyCustomClassFilter::OnOptionChanged(bool bIsEnabled)
	 *		{
	 *			// Updates the filter's config setting whenever the user changes the option.
	 *			MyFilterConfigSettings->bIsCustomFilterEnabled = bIsEnabled;
	 *		}
	 * 
	 *		bool FMyCustomClassFilter::IsClassAllowed(...)
	 *		{
	 *			if(!MyFilterConfigSettings->bIsCustomFilterEnabled)
	 *			{
	 *				// Filter is disabled; always allow the class.
	 *				return true;
	 *			}
	 * 
	 *			// ...
	 *		}
	 *
	 * @param OutFilterOptions		On output, contains the set of options to be added into the filter menu.
	 *								The Class Viewer requires these to be allocated so that each option can
	 *								be safely referenced by each menu item widget in the View Options menu.
	 */
	virtual void GetFilterOptions(TArray<TSharedRef<FClassViewerFilterOption>>& OutFilterOptions) {}
};

/** Filter class that performs many common checks. */
class FClassViewerFilter : public IClassViewerFilter
{
public:
	FClassViewerFilter(const FClassViewerInitializationOptions& InInitOptions);

	/**
	 * This function checks whether a node passes the filter defined by IsClassAllowed/IsUnloadedClassAllowed.
	 *
	 * @param InInitOptions				The Class Viewer/Picker options.
	 * @param Node						The Node to check.
	 * @param bCheckTextFilter			Whether to check the TextFilter. Disabling it could be useful e.g., to verify that the parent class of a IsNodeAllowed() object is also valid (regardless of the TextFilter, which will likely fail to pass).
	 */
	virtual bool IsNodeAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<FClassViewerNode>& Node, const bool bCheckTextFilter);

	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs ) override;
	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs, const bool bCheckTextFilter);
	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const class IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs) override;
	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const class IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs, const bool bCheckTextFilter);

	TArray<UClass*> InternalClasses;
	TArray<FDirectoryPath> InternalPaths;
	FOnShouldFilterClass FilterDelegate;
	TSharedRef<FTextFilterExpressionEvaluator> TextFilter;
	TSharedRef<FClassViewerFilterFuncs> FilterFunctions;
	TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter;
	const IAssetRegistry& AssetRegistry;
};

namespace EFilterReturn
{
	enum Type{ Failed = 0, Passed, NoItems };
}
class FClassViewerFilterFuncs
{
public:

	virtual ~FClassViewerFilterFuncs() = default;

	/** 
	 * Checks if the given Class is a child-of any of the classes in a set.
	 *
	 * @param InSet				The set to test against.
	 * @param InClass			The class to test against.
	 *
	 * @return					EFilterReturn::Passed if it is a child-of a class in the set, EFilterReturn::Failed if it is not, EFilterReturn::NoItems if the set is empty.
	 */
	UE_API virtual EFilterReturn::Type IfInChildOfClassesSet(TSet< const UClass* >& InSet, const UClass* InClass);

	/** 
	 * Checks if the given Class is a child-of any of the classes in a set.
	 *
	 * @param InSet				The set to test against.
	 * @param InClass			The class to test against.
	 *
	 * @return					EFilterReturn::Passed if it is a child-of a class in the set, EFilterReturn::Failed if it is not, EFilterReturn::NoItems if the set is empty.
	 */
	UE_API virtual EFilterReturn::Type IfInChildOfClassesSet(TSet< const UClass* >& InSet, const TSharedPtr< const class IUnloadedBlueprintData > InClass);

	/** 
	 * Checks if the given Class is a child-of ALL of the classes in a set.
	 *
	 * @param InSet				The set to test against.
	 * @param InClass			The class to test against.
	 *
	 * @return					EFilterReturn::Passed if it is a child-of ALL the classes in the set, EFilterReturn::Failed if it is not, EFilterReturn::NoItems if the set is empty.
	 */
	UE_API virtual EFilterReturn::Type IfMatchesAllInChildOfClassesSet(TSet< const UClass* >& InSet, const UClass* InClass);

	/** 
	 * Checks if the given Class is a child-of ALL of the classes in a set.
	 *
	 * @param InSet				The set to test against.
	 * @param InClass			The class to test against.
	 *
	 * @return					EFilterReturn::Passed if it is a child-of ALL the classes in the set, EFilterReturn::Failed if it is not, EFilterReturn::NoItems if the set is empty.
	 */
	UE_API virtual EFilterReturn::Type IfMatchesAllInChildOfClassesSet(TSet< const UClass* >& InSet, const TSharedPtr< const class IUnloadedBlueprintData > InClass);

	/** 
	 * Checks if ALL the Objects has a Is-A relationship with the passed in class.
	 *
	 * @param InSet				The set to test against.
	 * @param InClass			The class to test against.
	 *
	 * @return					EFilterReturn::Passed if ALL the Objects set IsA the passed class, EFilterReturn::Failed if it is not, EFilterReturn::NoItems if the set is empty.
	 */
	UE_API virtual EFilterReturn::Type IfMatchesAll_ObjectsSetIsAClass(TSet< const UObject* >& InSet, const UClass* InClass);

	/** 
	 * Checks if ALL the Objects has a Is-A relationship with the passed in class.
	 *
	 * @param InSet				The set to test against.
	 * @param InClass			The class to test against.
	 *
	 * @return					EFilterReturn::Passed if ALL the Objects set IsA the passed class, EFilterReturn::Failed if it is not, EFilterReturn::NoItems if the set is empty.
	 */
	UE_API virtual EFilterReturn::Type IfMatchesAll_ObjectsSetIsAClass(TSet< const UObject* >& InSet, const TSharedPtr< const class IUnloadedBlueprintData > InClass);

	/** 
	 * Checks if ALL the Classes set has a Is-A relationship with the passed in class.
	 *
	 * @param InSet				The set to test against.
	 * @param InClass			The class to test against.
	 *
	 * @return					EFilterReturn::Passed if ALL the Classes set IsA the passed class, EFilterReturn::Failed if it is not, EFilterReturn::NoItems if the set is empty.
	 */
	UE_API virtual EFilterReturn::Type IfMatchesAll_ClassesSetIsAClass(TSet< const UClass* >& InSet, const UClass* InClass);

	/** 
	 * Checks if ALL the Classes set has a Is-A relationship with the passed in class.
	 *
	 * @param InSet				The set to test against.
	 * @param InClass			The class to test against.
	 *
	 * @return					EFilterReturn::Passed if ALL the Classes set IsA the passed class, EFilterReturn::Failed if it is not, EFilterReturn::NoItems if the set is empty.
	 */
	UE_API virtual EFilterReturn::Type IfMatchesAll_ClassesSetIsAClass(TSet< const UClass* >& InSet, const TSharedPtr< const class IUnloadedBlueprintData > InClass);

	/** 
	 * Checks if any in the Classes set has a Is-A relationship with the passed in class.
	 *
	 * @param InSet				The set to test against.
	 * @param InClass			The class to test against.
	 *
	 * @return					EFilterReturn::Passed if the Classes set IsA the passed class, EFilterReturn::Failed if it is not, EFilterReturn::NoItems if the set is empty.
	 */
	UE_API virtual EFilterReturn::Type IfMatches_ClassesSetIsAClass(TSet< const UClass* >& InSet, const UClass* InClass);

	/** 
	 * Checks if any in the Classes set has a Is-A relationship with the passed in class.
	 *
	 * @param InSet				The set to test against.
	 * @param InClass			The class to test against.
	 *
	 * @return					EFilterReturn::Passed if the Classes set IsA the passed class, EFilterReturn::Failed if it is not, EFilterReturn::NoItems if the set is empty.
	 */
	UE_API virtual EFilterReturn::Type IfMatches_ClassesSetIsAClass(TSet< const UClass* >& InSet, const TSharedPtr< const class IUnloadedBlueprintData > InClass);

	/** 
	 * Checks if the Class is in the Classes set.
	 *
	 * @param InSet				The set to test against.
	 * @param InClass			The class to test against.
	 *
	 * @return					EFilterReturn::Passed if the Class is in the Classes set, EFilterReturn::Failed if it is not, EFilterReturn::NoItems if the set is empty.
	 */
	UE_API virtual EFilterReturn::Type IfInClassesSet(TSet< const UClass* >& InSet, const UClass* InClass);

	/** 
	 * Checks if the Class is in the Classes set.
	 *
	 * @param InSet				The set to test against.
	 * @param InClass			The class to test against.
	 *
	 * @return					EFilterReturn::Passed if the Class is in the Classes set, EFilterReturn::Failed if it is not, EFilterReturn::NoItems if the set is empty.
	 */
	UE_API virtual EFilterReturn::Type IfInClassesSet(TSet< const UClass* >& InSet, const TSharedPtr< const class IUnloadedBlueprintData > InClass);
};

class IUnloadedBlueprintData
{
public:

	/**
	 * Used to safely check whether the passed in flag is set.
	 *
	 * @param	InFlagsToCheck		Class flag to check for
	 *
	 * @return	true if the passed in flag is set, false otherwise
	 *			(including no flag passed in, unless the FlagsToCheck is CLASS_AllFlags)
	 */
	virtual bool HasAnyClassFlags( uint32 InFlagsToCheck ) const = 0;

	/**
	 * Used to safely check whether all of the passed in flags are set.
	 *
	 * @param InFlagsToCheck	Class flags to check for
	 * @return true if all of the passed in flags are set (including no flags passed in), false otherwise
	 */
	virtual bool HasAllClassFlags( uint32 InFlagsToCheck ) const = 0;

	/** 
	 * Sets the flags for this class.
	 *
	 * @param InFlags		The flags to be set to.
	 */
	virtual void SetClassFlags(uint32 InFlags) = 0;

	/** 
	 * This will return whether or not this class implements the passed in class / interface 
	 *
	 * @param InInterface - the interface to check and see if this class implements it
	 **/
	virtual bool ImplementsInterface(const UClass* InInterface) const = 0;

	/**
	 * Checks whether or not the class is a child-of the passed in class.
	 *
	 * @param InClass		The class to check against.
	 *
	 * @return				true if it is a child-of the passed in class.
	 */
	virtual bool IsChildOf(const UClass* InClass) const = 0;

	/** 
	 * Checks whether or not the class has an Is-A relationship with the passed in class.
	 *
	 * @param InClass		The class to check against.
	 *
	 * @return				true if it has an Is-A relationship to the passed in class.
	 */
	virtual bool IsA(const UClass* InClass) const = 0;

	/**
	 * Attempts to get the ClassWithin property for this class.
	 *
	 * @return ClassWithin of the child most Native class in the hierarchy.
	 */
	virtual const UClass* GetClassWithin() const = 0;

	/**
	 * Attempts to get the child-most Native class in the hierarchy.
	 *
	 * @return The child-most Native class in the hierarchy.
	 */
	virtual const UClass* GetNativeParent() const = 0;

	/** 
	 * Set whether or not this blueprint is a normal blueprint.
	 */
	virtual void SetNormalBlueprintType(bool bInNormalBPType) = 0;

	/** 
	 * Get whether or not this blueprint is a normal blueprint. 
	 */
	virtual bool IsNormalBlueprintType() const = 0;

	/**
	 * Get the generated class name of this blueprint.
	 */
	virtual TSharedPtr<FString> GetClassName() const = 0;

	/**
	 * Get the class path of this blueprint.
	 */
	virtual FTopLevelAssetPath GetClassPathName() const = 0;
};

#undef UE_API
