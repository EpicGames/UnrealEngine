// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class FClassViewerFilterFuncs;
class IClassViewerFilter;
class IPropertyHandle;
class SWidget;
class UClass;

CLASSVIEWER_API DECLARE_LOG_CATEGORY_EXTERN(LogEditorClassViewer, Log, All);

/** Delegate used with the Class Viewer in 'class picking' mode.  You'll bind a delegate when the
    class viewer widget is created, which will be fired off when a class is selected in the list */
DECLARE_DELEGATE_OneParam( FOnClassPicked, UClass* );

namespace EClassViewerMode
{
	enum Type
	{
		/** Allows all classes to be browsed and selected; syncs selection with the editor; drag and drop attachment, etc. */
		ClassBrowsing,

		/** Sets the class viewer to operate as a class 'picker'. */
		ClassPicker,
	};
}

namespace EClassViewerDisplayMode
{
	enum Type
	{
		/** Default will choose what view mode based on if in Viewer or Picker mode. */
		DefaultView,

		/** Displays all classes as a tree. */
		TreeView,

		/** Displays all classes as a list. */
		ListView,
	};
}

enum class EClassViewerNameTypeToDisplay : uint8
{
	/** Display both the display name and class name if they're available and different. */
	Dynamic,

	/** Always use the display name */
	DisplayName,

	/** Always use the class name */
	ClassName,
};

struct FClassViewerSortElementInfo
{
	FClassViewerSortElementInfo(TWeakObjectPtr<UClass> InClass, TSharedPtr<FString> InName, TSharedPtr<FString> InDisplayName)
		: Class(InClass), Name(InName), DisplayName(InDisplayName)
	{}
	
	TWeakObjectPtr<UClass> Class;

	TSharedPtr<FString> Name;

	TSharedPtr<FString> DisplayName;
};

/**
 * Settings for the Class Viewer set by the programmer before spawning an instance of the widget.  This
 * is used to modify the class viewer's behavior in various ways, such as filtering in or out specific classes.
 */
class FClassViewerInitializationOptions
{

public:
	/** The filter(s) to use on classes in this instance. */
	TArray<TSharedRef<IClassViewerFilter>> ClassFilters;

	/** Predicate used to sort the class list */
	TFunction<bool(const FClassViewerSortElementInfo&, const FClassViewerSortElementInfo&)> ClassViewerSortPredicate;

	/** Mode to operate in */
	EClassViewerMode::Type Mode;

	/** Mode to display the classes using. */
	EClassViewerDisplayMode::Type DisplayMode;

	/** Filters so only actors will be displayed. */
	bool bIsActorsOnly;

	/** Filters so only placeable actors will be displayed. Forces bIsActorsOnly to true. */
	bool bIsPlaceableOnly;

	/** Filters so only base blueprints will be displayed. */
	bool bIsBlueprintBaseOnly;

	/** Shows unloaded blueprints. Will not be filtered out based on non-bool filter options. */
	bool bShowUnloadedBlueprints;

	/** Shows a "None" option, only available in Picker mode. */
	bool bShowNoneOption;

	/** true will show the UObject root class. */
	bool bShowObjectRootClass;

	/** If true, root nodes will be expanded by default. */
	bool bExpandRootNodes;

	/** If true, all nodes will be expanded by default. */
	bool bExpandAllNodes;

	/** true allows class dynamic loading on selection */
	bool bEnableClassDynamicLoading;

	/** Controls what name is shown for classes */
	EClassViewerNameTypeToDisplay NameTypeToDisplay;

	/** the title string of the class viewer if required. */
	FText ViewerTitleString;

	/** The property this class viewer be working on. */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** The passed in property handle will be used to gather referencing assets. If additional referencing assets should be reported, supply them here. */
	TArray<FAssetData> AdditionalReferencingAssets;

	/** true (the default) shows the view options at the bottom of the class picker */
	bool bAllowViewOptions;

	/** true (the default) shows a background border behind the class viewer widget. */
	bool bShowBackgroundBorder = true;

	/** Defines additional classes you want listed in the "Common Classes" section for the picker. */
	TArray<UClass*> ExtraPickerCommonClasses;

	/** false by default, restricts the class picker to only showing editor module classes */
	bool bEditorClassesOnly;

	/** Will set the initially selected row, if possible, to this class when the viewer is created */
	UClass* InitiallySelectedClass;

	/** (true) Will show the default classes if they exist. */
	bool bShowDefaultClasses;

	/** (true) Will show the classes viewer. */
	bool bShowClassesViewer;

public:

	/** Constructor */
	FClassViewerInitializationOptions()	
		: Mode( EClassViewerMode::ClassPicker )
		, DisplayMode(EClassViewerDisplayMode::DefaultView)
		, bIsActorsOnly(false)
		, bIsPlaceableOnly(false)
		, bIsBlueprintBaseOnly(false)
		, bShowUnloadedBlueprints(true)
		, bShowNoneOption(false)
		, bShowObjectRootClass(false)
		, bExpandRootNodes(true)
		, bExpandAllNodes(false)
		, bEnableClassDynamicLoading(true)
		, NameTypeToDisplay(EClassViewerNameTypeToDisplay::ClassName)
		, bAllowViewOptions(true)
		, bEditorClassesOnly(false)
		, InitiallySelectedClass(nullptr)
		, bShowDefaultClasses(true)
		, bShowClassesViewer(true)
	{
	}
};

/**
 * Class Viewer module
 */
class FClassViewerModule : public IModuleInterface
{

public:

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule();

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule();

	/**
	 * Creates a class viewer widget
	 *
	 * @param	InitOptions						Programmer-driven configuration for this widget instance
	 * @param	OnClassPickedDelegate			Optional callback when a class is selected in 'class picking' mode
	 *
	 * @return	New class viewer widget
	 */
	virtual TSharedRef<SWidget> CreateClassViewer(const FClassViewerInitializationOptions& InitOptions, const FOnClassPicked& OnClassPickedDelegate );

	/** 
	 * Create a new class filter from the given initialization options.
	 */
	virtual TSharedRef<IClassViewerFilter> CreateClassFilter(const FClassViewerInitializationOptions& InitOptions);

	virtual TSharedRef<FClassViewerFilterFuncs> CreateFilterFuncs();

	/** Registers a global filter that affects all class viewer instances (gets combined any local filter)*/
	virtual void RegisterGlobalClassViewerFilter(const TSharedRef<IClassViewerFilter>& Filter);

	/** Returns the global filter that affects all class viewer instances */
	virtual const TSharedPtr<IClassViewerFilter>& GetGlobalClassViewerFilter();

	FSimpleMulticastDelegate& GetOnGlobalClassViewerFilterModified() { return OnGlobalClassViewerFilterModified; }

private:

	TSharedPtr<IClassViewerFilter> GlobalClassViewerFilter;

	FSimpleMulticastDelegate OnGlobalClassViewerFilterModified;
};
