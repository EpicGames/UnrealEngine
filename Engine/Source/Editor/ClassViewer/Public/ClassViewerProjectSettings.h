// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Engine/EngineTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"

#include "ClassViewerProjectSettings.generated.h"

struct FDirectoryPath;
struct FSoftClassPath;


/**
 * Implements the settings for the Class Viewer Project Settings
 */
UCLASS(MinimalAPI, config=Engine, defaultconfig)
class UClassViewerProjectSettings : public UObject
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	/** The base directories to be considered Internal Only for the class picker.*/
	UPROPERTY(EditAnywhere, config, Category = ClassVisibilityManagement, meta = (DisplayName = "List of directories to consider Internal Only.", ContentDir, LongPackageName))
	TArray<FDirectoryPath> InternalOnlyPaths;

	/** The base classes to be considered Internal Only for the class picker.*/
	UPROPERTY(EditAnywhere, config, Category = ClassVisibilityManagement, meta = (MetaClass = "/Script/CoreUObject.Object", DisplayName = "List of base classes to consider Internal Only.", AllowAbstract = "true", ShowTreeView, HideViewOptions))
	TArray<FSoftClassPath> InternalOnlyClasses;
#endif
};
