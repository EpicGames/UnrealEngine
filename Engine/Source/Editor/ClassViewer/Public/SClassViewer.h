// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClassViewerModule.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Engine/EngineTypes.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Optional.h"
#include "Settings/ClassViewerSettings.h"
#include "SlateFwd.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPath.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

class FClassViewerFilter;
class FClassViewerFilterOption;
class FClassViewerNode;
class FMenuBuilder;
class FString;
class FTextFilterExpressionEvaluator;
class ITableRow;
class SComboButton;
class SSearchBox;
class SWidget;
class UBlueprint;
class UClass;
struct FDirectoryPath;
struct FFocusEvent;
struct FGeometry;
struct FKeyEvent;
struct FPointerEvent;
struct FSoftClassPath;

//////////////////////////////////////////////////////////////////////////
// SClassViewer

class SClassViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SClassViewer )
	{
	}

	SLATE_ARGUMENT( FOnClassPicked, OnClassPickedDelegate )

	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 *
	 * @param	InArgs			A declaration from which to construct the widget
	 * @param	InitOptions		Programmer-driven initialization options for this widget
	 */
	void Construct(const FArguments& InArgs, const FClassViewerInitializationOptions& InInitOptions);

	/** Gets the widget contents of the app */
	virtual TSharedRef<SWidget> GetContent();

	virtual ~SClassViewer();

	/** Empty the selection set. */
	virtual void ClearSelection();

	/** SWidget interface */
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	virtual FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent ) override;
	virtual bool SupportsKeyboardFocus() const override;

	/** Test to see whether the given class would be allowed by this class viewer */
	virtual bool IsClassAllowed(const UClass* InClass) const;

	/** Sends a requests to the Class Viewer to refresh itself the next chance it gets */
	CLASSVIEWER_API void Refresh();

	/** Requests a repopulation of the internal Class Hierarchy database */
	CLASSVIEWER_API static void RequestPopulateClassHierarchy();

	/** Destroys the internal Class Hierarchy database */
	static void DestroyClassHierarchy();
private:
	/** Retrieves the children for the input node.
	 *	@param InParent				The parent node to retrieve the children from.
	 *	@param OutChildren			List of children for the parent node.
	 *
	 */
	void OnGetChildrenForClassViewerTree( TSharedPtr<FClassViewerNode> InParent, TArray< TSharedPtr< FClassViewerNode > >& OutChildren );

	/** Creates the row widget when called by Slate when an item appears on the tree. */
	TSharedRef< ITableRow > OnGenerateRowForClassViewer( TSharedPtr<FClassViewerNode> Item, const TSharedRef< STableViewBase >& OwnerTable );

	/** Invoked when the user attempts to drag an item out of the class browser */
	FReply OnDragDetected( const FGeometry& Geometry, const FPointerEvent& PointerEvent ) override;

	/** Called by Slate when the filter box changes text. */
	void OnFilterTextChanged( const FText& InFilterText );

	/** Called when enter is hit in search box */
	void OnFilterTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

	/** Called by Slate when an item is selected from the tree/list. */
	void OnClassViewerSelectionChanged( TSharedPtr<FClassViewerNode> Item, ESelectInfo::Type SelectInfo );

	/** Called by Slate when an item is expanded/collapsed from the tree/list. */
	void OnClassViewerExpansionChanged(TSharedPtr<FClassViewerNode> Item, bool bExpanded);

	/** A simple function for the filters to keep them enabled. */
	bool Menu_CanExecute() const;

	/** Toggles "Actors Only" true/false. */
	void MenuActorsOnly_Execute();

	/** Returns the state of "Actors Only". */
	bool MenuActorsOnly_IsChecked() const;

	/** Toggles "Placeable Only" true/false. */
	void MenuPlaceableOnly_Execute();

	/** Returns the state of "Placeable Only". */
	bool MenuPlaceableOnly_IsChecked() const;

	/** Toggles "Blueprint Bases Only" true/false. */
	void MenuBlueprintBasesOnly_Execute();

	/** Returns the state of "Blueprint Bases Only". */
	bool MenuBlueprintBasesOnly_IsChecked() const;

	/** Fills in the filter menu. */
	TSharedRef<SWidget> FillFilterEntries();

	/** 
	 *	Sets all expansion states in the tree.
	 *
	 *	@param bInExpansionState			The expansion state to set the tree to.
	 */
	void SetAllExpansionStates(bool bInExpansionState);

	/**
	 *	A helper function to recursively set the tree.
	 *
	 *	@param	InNode						The current node in the tree.
	 *	@param	bInExpansionState			The expansion state to set the tree to.
	 */
	void SetAllExpansionStates_Helper(TSharedPtr< FClassViewerNode > InNode, bool bInExpansionState);

	/**
	 *	A helper function to toggle expansion state of a single node
	 *
	 *	@param	InNode						The node to toggle expansion.
	 */
	void ToggleExpansionState_Helper(TSharedPtr< FClassViewerNode > InNode);

	/** Builds the right click menu widget for the selected node. */
	TSharedPtr< SWidget > BuildMenuWidget();

	/** Menu context option to open the currently selected Blueprint in the editor. */
	void OnOpenBlueprintTool();

	/** Menu context option to find the selected object in the content browser. */
	void FindInContentBrowser();

	/** Recursive function to expand nodes not filtered out of the tree
	*	@param	InNode				The current node to inspect for expansion.
	*
	*	@return bool				true if the child expanded, thus the parent should.
	*/
	bool ExpandFilteredInNodes(TSharedPtr<FClassViewerNode> InNode);

	/** Recursive function to map the expansion states of items in the tree. 
	 *	@param InItem		The current item to examine the expansion state of.
	 */
	void MapExpansionStatesInTree( TSharedPtr<FClassViewerNode> InItem );

	/** Recursive function to set the expansion states of items in the tree. 
	 *	@param InItem		The current item to set the expansion state of.
	 */
	void SetExpansionStatesInTree( TSharedPtr<FClassViewerNode> InItem );

	/** Populates the tree with items based on the current filter. */
	void Populate();

	/** Creates a "None" option for the tree/list. */
	TSharedPtr<FClassViewerNode> CreateNoneOption();

	/** Returns an array of the currently selected ClassViewerNodes */
	const TArray< TSharedPtr< FClassViewerNode > > GetSelectedItems() const;

	/** Expands all of the root nodes */
	virtual void ExpandRootNodes();



	/** Handler for when the view combo button is clicked */
	TSharedRef<SWidget> GetViewButtonContent();

	/** Gets the text for the class count label */
	FText GetClassCountText() const;

	/** Sets the view type and updates lists accordingly */
	void SetCurrentDeveloperViewType(EClassViewerDeveloperType NewType);

	/** Clears the reference to the current view and creates a new one, based on CurrentViewType */
	void CreateCurrentDeveloperView();

	/** Gets the current view type (list or tile) */
	EClassViewerDeveloperType GetCurrentDeveloperViewType() const;

	/** Returns true if ViewType is the current view type */
	bool IsCurrentDeveloperViewType(EClassViewerDeveloperType ViewType) const;

	/** Toggle whether internal use classes should be shown or not */
	void ToggleShowInternalClasses();

	/** Whether or not it's possible to show internal use classes */
	bool IsShowingInternalClasses() const;
	
	/** Whether or not it's possible to show internal use classes */
	bool IsToggleShowInternalClassesAllowed() const;

	/** Toggle whether a custom class filter option is enabled */
	void ToggleCustomFilterOption(TSharedRef<FClassViewerFilterOption> FilterOption);

	/** Returns whether a custom class filter option is currently enabled */
	bool IsCustomFilterOptionEnabled(TSharedRef<FClassViewerFilterOption> FilterOption) const;

	/** Get the total number of classes passing the current filters.*/
	const int GetNumItems() const;

	/** Count the number of tree items in the specified hierarchy*/
	int32 CountTreeItems(FClassViewerNode* Node);

	/** Handle the settings for ClassViewer changing.*/
	void HandleSettingChanged(FName PropertyName);

	/** Accessor for the classnames that have been marked as internal only in settings */
	void GetInternalOnlyClasses(TArray<FSoftClassPath>& Classes);
	
	/** Accessor for the class paths that have been marked as internal only in settings */
	void GetInternalOnlyPaths(TArray<FDirectoryPath>& Paths);
private:
	/** Init options, cached */
	FClassViewerInitializationOptions InitOptions;

	/** Filter to use to determine what classes are valid. */
	TSharedPtr<FClassViewerFilter> ClassFilter;

	/** The items to be displayed in the tree. */
	TArray< TSharedPtr< FClassViewerNode > > RootTreeItems;

	/** Holds the Slate Tree widget which holds the classes for the Class Viewer. */
	TSharedPtr<STreeView<TSharedPtr< FClassViewerNode > >> ClassTree;

	/** Holds the Slate List widget which holds the classes for the Class Viewer. */
	TSharedPtr<SListView<TSharedPtr< FClassViewerNode > >> ClassList;

	/** The Class Search Box, used for filtering the classes visible. */
	TSharedPtr<SSearchBox> SearchBox;

	/** true to allow class dynamic loading. */
	bool bEnableClassDynamicLoading;

	/** Callback that's fired when a class is selected while in 'class picking' mode */
	FOnClassPicked OnClassPicked;

	/** The class currently right clicked on. */
	UClass* RightClickClass;

	/** The blueprint currently right clicked on. */
	UBlueprint* RightClickBlueprint;

	/** The index of the class currently right clicked on. */
	int32 RightClickClassIndex;

	/** true if expansions states should be saved when compiling. */
	bool bSaveExpansionStates;

	/** The map holding the expansion state map for the tree. */
	TMap< FString, bool > ExpansionStateMap;

	/** True if the Class Viewer needs to be repopulated at the next appropriate opportunity, occurs whenever classes are added, removed, renamed, etc. */
	bool bNeedsRefresh;

	/** True if the search box will take keyboard focus next frame */
	bool bPendingFocusNextFrame;

	/** True if we need to set the tree expansion states according to our local copy next tick */
	bool bPendingSetExpansionStates;

	/** The button that displays view options */
	TSharedPtr<SComboButton> ViewOptionsComboButton;

	/** Number of classes that passed the filter*/
	int NumClasses;

	/** Custom filter options for the View Options menu */
	TArray<TSharedRef<FClassViewerFilterOption>> CustomClassFilterOptions;
};
